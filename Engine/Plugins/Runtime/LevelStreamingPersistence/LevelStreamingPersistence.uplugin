{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.2",
	"FriendlyName": "Level Streaming Persistence",
	"Description": "An experimental framework for persisting world state: actor and subobject property values, actor destroyed states, and actor respawning. State is associated with streaming levels, and optionally the persistent level.",
	"Category": "",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "LevelStreamingPersistence",
			"Type": "Runtime",
			"LoadingPhase": "PreLoadingScreen"
		}
	],
	"Plugins": [
	]
}