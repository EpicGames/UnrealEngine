// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LevelStreamingPersistence : ModuleRules
	{
		public LevelStreamingPersistence(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"PropertyPath",
					"DeveloperSettings"
				}
			);
		}
	}
}