// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelStreamingPersistenceManager.h"
#include "LevelStreamingPersistenceModule.h"
#include "LevelStreamingPersistenceSettings.h"
#include "LevelStreamingPersistenceLog.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Streaming/LevelStreamingDelegates.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "StructUtils/PropertyBag.h"
#include "PropertyPathHelpers.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/Engine.h"
#include "LevelUtils.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "GameFramework/WorldSettings.h"
#include "UObject/Package.h"

#if !UE_BUILD_SHIPPING
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelStreamingPersistenceManager)

struct FLevelStreamingPersistenceCustomVersion
{
	enum Type
	{
		BeforeCustomVersionWasAdded = 0,

		// Added RemovedMapActors, RuntimeSpawnedActors to FLevelStreamingPersistentPropertyValues.
		// Added RuntimeSpawnedActorClassPaths to ULevelStreamingPersistenceManager payload.
		RuntimeActorPersistence,

		// -----<new versions can be added above this line>-----
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	const static FGuid GUID;
};

const FGuid FLevelStreamingPersistenceCustomVersion::GUID(0xB4E2D1C0, 0x5A3F8B7E, 0x92461035, 0xD7F8E9A0);
FCustomVersionRegistration GRegisterLevelStreamingPersistenceCustomVersion(
	FLevelStreamingPersistenceCustomVersion::GUID,
	FLevelStreamingPersistenceCustomVersion::LatestVersion,
	TEXT("LevelStreamingPersistence")
);

#define LOCTEXT_NAMESPACE "LevelStreamingPersistence"

/*
 * Achives used by ULevelStreamingPersistenceManager
 */

// By default, LevelStreamingPersistenceManager uses tagged property serialization, which is more flexible as it
// allows to load a previously serialized snapshot of ULevelStreamingPersistenceManager even if 
// LevelStreamingPersistenceSettings Properties has changed afterwards.
// Setting this flag to false will use less memory, with the disadvantage of not being resilient to changes
static const bool GUseTaggedPropertySerialization = true;

class FPersistentPropertiesArchive : public FObjectAndNameAsStringProxyArchive
{
public:
	FPersistentPropertiesArchive(FArchive& InArchive, const TArray<const FProperty*>& InPersistentProperties)
		: FObjectAndNameAsStringProxyArchive(InArchive, /*bLoadIfFindFails*/false)
		, PersistentProperties(InPersistentProperties)
	{
		check(InArchive.IsPersistent());
		check(InArchive.IsFilterEditorOnly());
		check(InArchive.ShouldSkipBulkData());
		SetIsLoading(InArchive.IsLoading());
		SetIsSaving(InArchive.IsSaving());
		SetIsTextFormat(InArchive.IsTextFormat());
		SetWantBinaryPropertySerialization(InArchive.WantBinaryPropertySerialization());
		SetIsPersistent(true);
		FArchiveProxy::SetFilterEditorOnly(true);
		ArShouldSkipBulkData = true;

		if (WantBinaryPropertySerialization())
		{
			// Custom property lists only work with binary serialization, not tagged property serialization.
			BuildSerializedPropertyList();
			ArUseCustomPropertyList = !CustomPropertyList.IsEmpty();
			ArCustomPropertyList = ArUseCustomPropertyList ? &CustomPropertyList[0] : nullptr;
		}
	}

	virtual bool ShouldSkipProperty(const FProperty* InProperty) const
	{
		if (FObjectAndNameAsStringProxyArchive::ShouldSkipProperty(InProperty))
		{
			return true;
		}

		if (!ArUseCustomPropertyList)
		{
			return !PersistentProperties.Contains(InProperty);
		}

		return false;
	}

	virtual FArchive& operator<<(FLazyObjectPtr& Value) override { return FArchiveUObject::SerializeLazyObjectPtr(*this, Value); }

private:

	void BuildSerializedPropertyList()
	{
		for (const FProperty* Property : PersistentProperties)
		{
			CustomPropertyList.Emplace(const_cast<FProperty*>(Property));
		}

		// Link changed properties
		if (!CustomPropertyList.IsEmpty())
		{
			for (int i = 0; i < CustomPropertyList.Num() - 1; ++i)
			{
				CustomPropertyList[i].PropertyListNext = &CustomPropertyList[i + 1];
			}
		}
	}

	const TArray<const FProperty*>& PersistentProperties;
	FLevelStreamingPersistentPropertyArray CustomPropertyList;
};

class FPersistentPropertiesWriter : public FMemoryWriter
{
public:
	FPersistentPropertiesWriter(bool bInUseTaggedPropertySerialization, TArray<uint8, TSizedDefaultAllocator<32>>& InBytes)
		: FMemoryWriter(InBytes, true)
	{
		SetFilterEditorOnly(true);
		ArShouldSkipBulkData = true;
		SetIsTextFormat(false);
		SetWantBinaryPropertySerialization(!bInUseTaggedPropertySerialization);
	}
};

class FPersistentPropertiesReader : public FMemoryReader
{
public:
	FPersistentPropertiesReader(bool bInUseTaggedPropertySerialization, const TArray<uint8>& InBytes)
		: FMemoryReader(InBytes, true)
	{
		SetFilterEditorOnly(true);
		ArShouldSkipBulkData = true;
		SetIsTextFormat(false);
		SetWantBinaryPropertySerialization(!bInUseTaggedPropertySerialization);
	}
};

/*
 * ULevelStreamingPersistenceManager implementation
 */

ULevelStreamingPersistenceManager::ULevelStreamingPersistenceManager(const FObjectInitializer& ObjectInitializer)
	: PersistenceModule(&ILevelStreamingPersistenceModule::Get())
	, bUseTaggedPropertySerialization(GUseTaggedPropertySerialization)
{
}

bool ULevelStreamingPersistenceManager::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	return InWorldType == EWorldType::PIE || InWorldType == EWorldType::Game;
}

bool ULevelStreamingPersistenceManager::ShouldCreateSubsystem(UObject* Outer) const
{
	if (!UWorldSubsystem::ShouldCreateSubsystem(Outer))
	{
		return false;
	}

	// Created for any non-client game world. Map-actor destruction, runtime respawn, and
	// optional persistent-level state all work without WorldPartition streaming, so we no
	// longer gate on it.
	UWorld* World = Cast<UWorld>(Outer);
	if (!World || !World->IsGameWorld() || (World->GetNetMode() == NM_Client))
	{
		return false;
	}

	return true;
}

void ULevelStreamingPersistenceManager::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	FLevelStreamingDelegates::OnLevelBeginMakingInvisible.AddUObject(this, &ULevelStreamingPersistenceManager::OnLevelBeginMakingInvisible);
	FLevelStreamingDelegates::OnLevelBeginMakingVisible.AddUObject(this, &ULevelStreamingPersistenceManager::OnLevelBeginMakingVisible);
	FWorldDelegates::LevelAddedToWorld.AddUObject(this, &ULevelStreamingPersistenceManager::OnLevelAddedToWorld);

	// Listen to persistent world initialization to restore properties on it
	const ULevelStreamingPersistenceSettings* Settings = GetDefault<ULevelStreamingPersistenceSettings>();
	if (Settings->ShouldIncludePersistentLevel())
	{
		GetWorld()->OnActorsInitialized.AddUObject(this, &ULevelStreamingPersistenceManager::OnWorldActorsInitialized);
	}

	// Listen to actor destruction to persist paths to map actors that have been destroyed
	if (Settings->ShouldPersistAnyActorDestruction())
	{
		ActorDestroyedHandle = GetWorld()->AddOnActorDestroyedHandler(FOnActorDestroyed::FDelegate::CreateUObject(this, &ULevelStreamingPersistenceManager::OnActorDestroyed));
	}

	PersistentPropertiesInfo = NewObject<ULevelStreamingPersistentPropertiesInfo>(this);
	PersistentPropertiesInfo->Initialize();
}

void ULevelStreamingPersistenceManager::Deinitialize()
{
	FLevelStreamingDelegates::OnLevelBeginMakingInvisible.RemoveAll(this);
	FLevelStreamingDelegates::OnLevelBeginMakingVisible.RemoveAll(this);
	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);

	if (UWorld* World = GetWorld())
	{
		World->OnActorsInitialized.RemoveAll(this);
		if (ActorDestroyedHandle.IsValid())
		{
			World->RemoveOnActorDestroyedHandler(ActorDestroyedHandle);
		}
	}
}

bool ULevelStreamingPersistenceManager::TrySetPropertyValueFromString(const FString& InObjectPathName, const FName InPropertyName, const FString& InPropertyValue)
{
	const FString ObjectPathName = GetResolvedObjectPathName(InObjectPathName);
	FLevelStreamingPersistentObjectPropertyBag* PropertyBag = GetPropertyBag(ObjectPathName);
	if (PropertyBag && PropertyBag->SetPropertyValueFromString(InPropertyName, InPropertyValue))
	{
		// If object is loaded, copy property value to object
		auto ObjectPropertyPair = GetObjectPropertyPair(ObjectPathName, InPropertyName);
		CopyPropertyBagValueToObject(PropertyBag, ObjectPropertyPair.Key, ObjectPropertyPair.Value);
		return true;
	}
	return false;
}

bool ULevelStreamingPersistenceManager::GetPropertyValueAsString(const FString& InObjectPathName, const FName InPropertyName, FString& OutPropertyValue)
{
	const FString ObjectPathName = GetResolvedObjectPathName(InObjectPathName);
	if (FLevelStreamingPersistentObjectPropertyBag* PropertyBag = GetPropertyBag(ObjectPathName))
	{
		return PropertyBag->GetPropertyValueAsString(InPropertyName, OutPropertyValue);
	}
	return false;
}

void ULevelStreamingPersistenceManager::OnLevelBeginMakingInvisible(UWorld* World, const ULevelStreaming* InStreamingLevel, ULevel* InLoadedLevel)
{
	if (IsValid(InLoadedLevel) && (World == GetWorld()) && IsEnabled())
	{
		// Cancel any deferred level restore tasks for this level.
		LevelsPendingFinishRestore.Remove(InLoadedLevel->GetPathName());

		SaveLevelPersistentPropertyValues(InLoadedLevel);
	}
}

bool ULevelStreamingPersistenceManager::SaveLevelPersistentPropertyValues(const ULevel * InLevel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULevelStreamingPersistenceManager::SaveLevelPersistentPropertyValues);

	check(IsValid(InLevel) && IsEnabled());
	const FString LevelPathName = InLevel->GetPathName();
	FLevelStreamingPersistentPropertyValues& LevelProperties = LevelsPropertyValues.FindOrAdd(LevelPathName);

	// Fire level-wide pre-persist hook.
	PersistenceModule->PrePersistLevel.ExecuteIfBound(InLevel);

	// Fire per-object pre-persist hook once for each valid persistent object before any serialization begins.
	// This allows game code to prepare persistent data for serialization, like representing gameplay state
	// (inventory, buffs, AI state) as serializable properties.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SaveLevelPersistentPropertyValues: PrePersistObjects);
		TSet<const UObject*> UniqueObjects;
		for (const auto& [ObjectPath, ObjectPrivatePropertyValues] : LevelProperties.ObjectsPrivatePropertyValues)
		{
			const UObject* Object = FindObject<UObject>(nullptr, *ObjectPath);
			if (IsValid(Object))
			{
				UniqueObjects.Add(Object);
			}
		}
		for (const auto& [ObjectPath, ObjectPublicPropertyValues] : LevelProperties.ObjectsPublicPropertyValues)
		{
			const UObject* Object = FindObject<UObject>(nullptr, *ObjectPath);
			if (IsValid(Object))
			{
				UniqueObjects.Add(Object);
			}
		}
		for (const UObject* Object : UniqueObjects)
		{
			PersistenceModule->PrePersistObject(Object);
		}
	}

	auto SavePrivateProperties = [this, &LevelProperties]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SaveLevelPersistentPropertyValues: SavePrivateProperties);
		int32 SavedCount = 0;
		for (auto& [ObjectPath, ObjectPrivatePropertyValues] : LevelProperties.ObjectsPrivatePropertyValues)
		{
			if (const UObject* Object = FindObject<UObject>(nullptr, *ObjectPath); IsValid(Object))
			{
				ObjectPrivatePropertyValues.PayloadData.Reset();
				ObjectPrivatePropertyValues.PersistentProperties.Reset();
				if (DiffWithSnapshot(Object, ObjectPrivatePropertyValues.Snapshot, ObjectPrivatePropertyValues.PersistentProperties))
				{
					FPersistentPropertiesWriter WriterAr(bUseTaggedPropertySerialization, ObjectPrivatePropertyValues.PayloadData);
					FPersistentPropertiesArchive Ar(WriterAr, ObjectPrivatePropertyValues.PersistentProperties);
					Object->SerializeScriptProperties(Ar);
					SavedCount += (ObjectPrivatePropertyValues.PersistentProperties.Num() > 0) ? 1 : 0;
				}
			}
		}
		return SavedCount;
	};

	auto SavePublicProperties = [this, &LevelProperties]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SaveLevelPersistentPropertyValues: SavePublicProperties);
		int32 SavedCount = 0;
		for (auto& [ObjectPath, ObjectPublicPropertyValues] : LevelProperties.ObjectsPublicPropertyValues)
		{
			if (const UObject* Object = FindObject<UObject>(nullptr, *ObjectPath); IsValid(Object))
			{
				if (ensure(ObjectPublicPropertyValues.PropertyBag.IsValid()))
				{
					ObjectPublicPropertyValues.PersistentProperties.Reset();
					for (const FProperty* BagProperty : ObjectPublicPropertyValues.PropertiesToPersist)
					{
						if (FProperty* ObjectProperty = Object->GetClass()->FindPropertyByName(BagProperty->GetFName()))
						{
							if (ObjectPublicPropertyValues.PropertyBag.CopyPropertyValueFromObject(Object, ObjectProperty))
							{
								ObjectPublicPropertyValues.PersistentProperties.Add(BagProperty);
								++SavedCount;
							}
						}
					}
				}
			}
		}
		return SavedCount;
	};

	auto SaveRuntimeSpawnedActors = [this, &InLevel, &LevelProperties]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SaveLevelPersistentPropertyValues: SaveRuntimeSpawnedActors);
		// Forget any cached respawn records
		LevelProperties.RuntimeSpawnedActors.Reset();

		// Abort early if no runtime respawnable classes are configured
		const ULevelStreamingPersistenceSettings* Settings = GetDefault<ULevelStreamingPersistenceSettings>();
		if (!Settings->ShouldRuntimeRespawnAnyActor())
		{
			return 0;
		}

		// Iterate actors in the level to detect runtime respawnable actors
		int32 SavedCount = 0;
		for (AActor* Actor : InLevel->Actors)
		{
			// Skip invalid actors
			if (!IsValid(Actor))
			{
				continue;
			}

			// Map actors aren't treated as respawnable: they will simply be reloaded with the level.
			// The only exception is if the actor is "ejected": the actor lives but game code has explicitly marked it for removal on reload.
			// Then, the actor is considered for runtime respawning. This allows a map placed actor to cross world partition boundaries.
			if (Actor->IsNetStartupActor() && !LevelProperties.RemovedMapActors.Contains(Actor->GetFName()))
			{
				continue;
			}

			// Skip actor classes that aren't marked as runtime respawnable
			if (!Settings->ShouldPersistRuntimeActorClass(Actor->GetClass()))
			{
				continue;
			}

			if (!PersistenceModule->ShouldPersistRuntimeActor(Actor))
			{
				continue;
			}

			// Create a respawn record for this actor
			FRuntimeSpawnedActorRecord& Record = LevelProperties.RuntimeSpawnedActors.AddDefaulted_GetRef();
			BuildRuntimeSpawnedActorRecord(Actor, Record);
			++SavedCount;
		}
		return SavedCount;
	};

	const int32 SavedCount = SavePrivateProperties() + SavePublicProperties() + SaveRuntimeSpawnedActors();
	return SavedCount > 0;
}

void ULevelStreamingPersistenceManager::UpdateVisibleLevelsPersistentPropertyValues()
{
	if (IsEnabled())
	{
		// Iterate all of the world's levels
		UWorld* World = GetWorld();
		const bool bIncludePersistentLevel = GetDefault<ULevelStreamingPersistenceSettings>()->ShouldIncludePersistentLevel();
		for (ULevel* Level : World->GetLevels())
		{
			// Saving anything for the persistent level is opt-in
			if (Level->IsPersistentLevel() && !bIncludePersistentLevel)
			{
				continue;
			}

			// Only refresh visible levels. If the level is not visible, it's state should be up-to-date already.
			// It would have been updated when the level was made invisible.
			if (Level->bIsVisible)
			{
				// Only refresh levels if they already have an entry in LevelsPropertyValues. If the level is
				// eligible for persistence, an entry for it should have been made when it became visible.
				const FString LevelPathName = Level->GetPathName();
				if (LevelsPropertyValues.Contains(LevelPathName))
				{
					SaveLevelPersistentPropertyValues(Level);
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "Call to UpdateVisibleLevelsPersistentPropertyValues ignored because persistence is disabled by cvar");
	}
}

void ULevelStreamingPersistenceManager::RestoreVisibleLevelsPersistentPropertyValues()
{
	// We've just deserialized persistent info. InitializeFrom can be called by game code and the timing isn't strict.
	// If this is called early during world construction (ideal) like another world subsystem initialize, then World->OnActorsInitialized
	// will apply the data later on the persistent level and OnLevelBeginMakingVisible will apply it to streamed in levels, including
	// initial visible ones. In case game code called InitializeFrom after those have already happened, apply level saved data to those
	// visible levels now, immediately. Properties will be restored, but actors may have BegunPlay already.
	if (IsEnabled())
	{
		for (const TPair<FString, FLevelStreamingPersistentPropertyValues>& LevelPair : LevelsPropertyValues)
		{
			if (ULevel* Level = FindObject<ULevel>(nullptr, *LevelPair.Key))
			{
				if (IsValid(Level) && Level->bIsVisible)
				{
					RestoreLevelPersistentPropertyValues(Level);
				}
			}
		}
	}
}

void ULevelStreamingPersistenceManager::OnWorldActorsInitialized(const FActorsInitializedParams& Params)
{
	if (!IsEnabled())
	{
		return;
	}

	// This callback should only be registered if game opted into including the persistent level
	ensure(GetDefault<ULevelStreamingPersistenceSettings>()->ShouldIncludePersistentLevel());

	// Restore persistent properties for actors in the persistent level. Index them with their current 
	// initial values, if they didn't have records yet.
	UWorld* World = GetWorld();
	if (IsValid(World) && IsValid(World->PersistentLevel))
	{
		RestoreLevelPersistentPropertyValues(World->PersistentLevel);
	}
}

void ULevelStreamingPersistenceManager::OnActorDestroyed(AActor* DestroyedActor)
{
	// Only map placed actors are tracked for removal on reload
	if (DestroyedActor && DestroyedActor->IsNetStartupActor() && IsEnabled())
	{
		// Don't track actor destruction if the level was not in play
		ULevel* FromLevel = DestroyedActor->GetLevel();
		if (!FromLevel || !FromLevel->bIsVisible)
		{
			return;
		}

		// Streaming levels eligible for persistence have FLevelStreamingPersistentPropertyValues indexed already.
		// Only track destruction if the level already has an entry in LevelsPropertyValues.
		const FName ActorName = DestroyedActor->GetFName();
		const FString LevelPathName = FromLevel->GetPathName();
		FLevelStreamingPersistentPropertyValues* LevelProperties = LevelsPropertyValues.Find(LevelPathName);
		if (LevelProperties && !LevelProperties->RemovedMapActors.Contains(ActorName))
		{
			// Only track configured actor classes for destruction
			const ULevelStreamingPersistenceSettings* Settings = GetDefault<ULevelStreamingPersistenceSettings>();
			if (Settings && Settings->ShouldPersistActorDestruction(DestroyedActor))
			{
				LevelProperties->RemovedMapActors.Add(ActorName);
				UE_LOGF(LogLevelStreamingPersistence, Verbose, "Marking destroyed actor for removal on reload: %ls", *DestroyedActor->GetName());
			}
		}
	}
}

int32 ULevelStreamingPersistenceManager::ReapplyRemovedActors(FLevelStreamingPersistentPropertyValues* LevelProperties, ULevel* Level, const bool bMarkAsGarbage)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULevelStreamingPersistenceManager::ReapplyRemovedActors);

	int32 Count = 0;
	const ULevelStreamingPersistenceSettings* Settings = GetDefault<ULevelStreamingPersistenceSettings>();

	// Re-destroy the actors that were marked to be removed from a previous session.
	// Iterate on a copy of the set, in case one actor's destruction triggers more
	// destructions or marking for removal (i.e. eject). We want those additional
	// destructions to be remembered as well, but don't need to process them now.
	TSet<FName> RemovedMapActorsCopy(LevelProperties->RemovedMapActors);
	for (const FName& ActorName : RemovedMapActorsCopy)
	{
		AActor* Actor = Cast<AActor>(StaticFindObjectFast(AActor::StaticClass(), Level, ActorName));
		if (IsValid(Actor) && Settings->ShouldPersistActorDestruction(Actor))
		{
			if (bMarkAsGarbage)
			{
				UE_LOGF(LogLevelStreamingPersistence, Verbose, "Marking actor as garbage from previous session: %ls", *Actor->GetName());
				Actor->MarkAsGarbage();
			}
			else
			{
				UE_LOGF(LogLevelStreamingPersistence, Verbose, "Re-destroying actor from previous session: %ls", *Actor->GetName());
				Actor->Destroy();
			}
			++Count;
		}
	}
	return Count;
}

void ULevelStreamingPersistenceManager::OnLevelAddedToWorld(ULevel* Level, UWorld* World)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULevelStreamingPersistenceManager::OnLevelAddedToWorld);

	if (!IsValid(Level) || World != GetWorld() || !IsEnabled())
	{
		return;
	}

	const FString LevelPathName = Level->GetPathName();
	FLevelStreamingPersistentPropertyValues* LevelProperties = LevelsPropertyValues.Find(LevelPathName);
	if (LevelProperties && LevelsPendingFinishRestore.Remove(LevelPathName))
	{
		// Now that the level is visible, perform pending restoration
		ReapplyRemovedActors(LevelProperties, Level, /*bMarkAsGarbage=*/false);

		// Only respawn if the level was GCed and reloaded. If the same ULevel is still in memory,
		// the runtime actors are still alive and don't need to be respawned.
		if (LevelProperties->LastKnownLevel != Level)
		{
			RestoreRuntimeSpawnedActors(LevelProperties, Level);
		}

		// Finished restoring. Fire level-wide post-restore hook.
		PersistenceModule->PostRestoreLevel.ExecuteIfBound(Level);

		// Store a weak reference to the level we just finished restoring on. Respawn tasks aren't 
		// applied to this level again when it becomes invisible and visible. It only happens again 
		// if the level gets GCed and reloaded.
		LevelProperties->LastKnownLevel = Level;
	}
}

void ULevelStreamingPersistenceManager::OnLevelBeginMakingVisible(UWorld* World, const ULevelStreaming* InStreamingLevel, ULevel* InLoadedLevel)
{
	if (IsValid(InLoadedLevel) && (World == GetWorld()) && IsEnabled())
	{
		RestoreLevelPersistentPropertyValues(InLoadedLevel);
	}
}

bool ULevelStreamingPersistenceManager::RestoreLevelPersistentPropertyValues(ULevel* InLevel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULevelStreamingPersistenceManager::RestoreLevelPersistentPropertyValues);

	check(IsValid(InLevel) && IsEnabled());

	auto CreatePrivatePropertiesSnapshot = [this](UObject* Object, FLevelStreamingPersistentPropertyValues& LevelProperties)
	{
		int CreatedCount = 0;
		const UClass* ObjectClass = Object->GetClass();
		if (PersistentPropertiesInfo->HasProperties(ULevelStreamingPersistentPropertiesInfo::PropertyType_Private, ObjectClass))
		{
			const FString ObjectPathName = Object->GetPathName();
			FLevelStreamingPersistentObjectPrivateProperties* ObjectPrivatePropertyValues = LevelProperties.ObjectsPrivatePropertyValues.Find(ObjectPathName);
			if (!ObjectPrivatePropertyValues || !ObjectPrivatePropertyValues->Snapshot.IsValid())
			{
				if (!ObjectPrivatePropertyValues)
				{
					ObjectPrivatePropertyValues = &LevelProperties.ObjectsPrivatePropertyValues.Add(ObjectPathName);
					ObjectPrivatePropertyValues->SourceClassPathName = ObjectClass->GetPathName();
				}
			}
			
			if (!ObjectPrivatePropertyValues->Snapshot.IsValid() && BuildSnapshot(Object, ObjectPrivatePropertyValues->Snapshot))
			{
				++CreatedCount;
			}
			
			// Clean-up private entries
			if (!ObjectPrivatePropertyValues->Snapshot.IsValid())
			{
				LevelProperties.ObjectsPrivatePropertyValues.Remove(ObjectPathName);
			}
		}
		return CreatedCount;
	};

	auto CreatePublicPropertiesEntry = [this](const UObject* Object, FLevelStreamingPersistentPropertyValues& LevelProperties)
	{
		int32 CreatedCount = 0;
		check(IsValid(Object));
		const UClass* ObjectClass = Object->GetClass();

		if (PersistentPropertiesInfo->HasProperties(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, ObjectClass))
		{
			const FString ObjectPathName = Object->GetPathName();
			FLevelStreamingPersistentObjectPublicProperties* ObjectPublicPropertyValues = LevelProperties.ObjectsPublicPropertyValues.Find(ObjectPathName);

			// Build PropertiesToPersist if necessary
			TArray<FProperty*, TInlineAllocator<32>> SerializedObjectProperties;
			if (!ObjectPublicPropertyValues || ObjectPublicPropertyValues->PropertiesToPersist.IsEmpty())
			{
				PersistentPropertiesInfo->ForEachProperty(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, ObjectClass, [this, Object, &SerializedObjectProperties](FProperty* ObjectProperty)
				{
					if (PersistenceModule->ShouldPersistProperty(Object, ObjectProperty)
						&& PersistentPropertiesInfo->SatisfiesFilters(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, ObjectProperty, Object))
					{
						SerializedObjectProperties.Add(ObjectProperty);
					}
				});

				if (!SerializedObjectProperties.IsEmpty())
				{
					check(!ObjectPublicPropertyValues || ObjectPublicPropertyValues->PropertyBag.IsValid());
					if (!ObjectPublicPropertyValues)
					{
						// Create entry if necessary and initialize PropertyBag
						ObjectPublicPropertyValues = &LevelProperties.ObjectsPublicPropertyValues.Add(ObjectPathName);
						ObjectPublicPropertyValues->SourceClassPathName = ObjectClass->GetPathName();
						ObjectPublicPropertyValues->PropertyBag.Initialize([this, ObjectClass]() { return PersistentPropertiesInfo->GetPropertyBagFromClass(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, ObjectClass); });
					}
					if (ObjectPublicPropertyValues->PropertyBag.IsValid())
					{
						check(ObjectPublicPropertyValues->PropertiesToPersist.IsEmpty());
						for (FProperty* ObjectProperty : SerializedObjectProperties)
						{
							if (const FProperty* BagProperty = ObjectPublicPropertyValues->PropertyBag.GetCompatibleProperty(ObjectProperty))
							{
								ObjectPublicPropertyValues->PropertiesToPersist.Add(BagProperty);
								++CreatedCount;
							}
						}
					}
				}
			}

			// Clean-up public entries
			if (ObjectPublicPropertyValues && (!ObjectPublicPropertyValues->PropertyBag.IsValid() || ObjectPublicPropertyValues->PropertiesToPersist.IsEmpty()))
			{
				LevelProperties.ObjectsPublicPropertyValues.Remove(ObjectPathName);
			}
		}
		return CreatedCount;
	};

	int32 CreatedCount = 0;
	const FString LevelPathName = InLevel->GetPathName();
	FLevelStreamingPersistentPropertyValues* LevelProperties = LevelsPropertyValues.Find(LevelPathName);
	const bool bIsNewLevelEntry = (LevelProperties == nullptr);

	// If the level didn't have persistent data cached yet, create an initial snapshot for it now.
	// Or, if the cache was marked outdated (i.e. a cache was loaded from older serialized data),
	// refresh the cache.
	if (bIsNewLevelEntry || !LevelProperties->bIsMakingVisibleCacheValid)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ULevelStreamingPersistenceManager::CreateLevelPropertiesEntries);

		FLevelStreamingPersistentPropertyValues& NewLevelProperties = LevelsPropertyValues.FindOrAdd(LevelPathName);
		LevelProperties = &NewLevelProperties;

		TSet<FString> ValidObjects;
		ForEachObjectWithOuter(InLevel, [this, bIsNewLevelEntry, &NewLevelProperties, &ValidObjects, &CreatedCount, &CreatePrivatePropertiesSnapshot, &CreatePublicPropertiesEntry](UObject* Object)
		{
			int32 EntryCreatedCount = CreatePrivatePropertiesSnapshot(Object, NewLevelProperties);
			EntryCreatedCount += CreatePublicPropertiesEntry(Object, NewLevelProperties);
			if (!bIsNewLevelEntry && (EntryCreatedCount > 0))
			{
				ValidObjects.Add(Object->GetPathName());
			}
			CreatedCount += EntryCreatedCount;
		}, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage);

		// Clean-up object entries that don't exist in the level
		if (ValidObjects.Num() > 0)
		{
			for (auto It = LevelProperties->ObjectsPrivatePropertyValues.CreateIterator(); It; ++It)
			{
				if (!ValidObjects.Contains(It.Key()))
				{
					It.RemoveCurrent();
				}
			}
			for (auto It = LevelProperties->ObjectsPublicPropertyValues.CreateIterator(); It; ++It)
			{
				if (!ValidObjects.Contains(It.Key()))
				{
					It.RemoveCurrent();
				}
			}
		}

		// Mark level entry as valid
		LevelProperties->bIsMakingVisibleCacheValid = true;
	}

	int32 RestoredCount = 0;
	int32 DestroyedCount = 0;
	int32 SpawnedCount = 0;
	if (!bIsNewLevelEntry)
	{
		// Restore persistent properties on map placed objects. Track overwritten properties to pass as a list into post-restore hooks.
		TMap<UObject*, TArray<const FProperty*>> RestoredPropertiesMap;
		for (const auto& [ObjectPath, ObjectPrivatePropertyValues] : LevelProperties->ObjectsPrivatePropertyValues)
		{
			// Find the object by path and only restore properties if it's valid.
			// Checking validity because an object may exist in memory, yet may have been destroyed during gameplay
			// before the level was streamed out. We don't want to restore properties on those stale objects,
			// they're just pending GC.
			UObject* Object = FindObject<UObject>(nullptr, *ObjectPath);
			if (IsValid(Object))
			{
				TArray<const FProperty*>& RestoredProperties = RestoredPropertiesMap.FindOrAdd(Object);
				RestoredCount += RestorePrivateProperties(Object, ObjectPrivatePropertyValues, RestoredProperties);
			}
		}
		for (const auto& [ObjectPath, ObjectPublicPropertyValues] : LevelProperties->ObjectsPublicPropertyValues)
		{
			UObject* Object = FindObject<UObject>(nullptr, *ObjectPath);
			if (IsValid(Object))
			{
				TArray<const FProperty*>& RestoredProperties = RestoredPropertiesMap.FindOrAdd(Object);
				RestoredCount += RestorePublicProperties(Object, ObjectPublicPropertyValues, RestoredProperties);
			}
		}

		// Fire post-restore object hooks, once per object with all its restored properties.
		for (const auto& [Object, RestoredProperties] : RestoredPropertiesMap)
		{
			PersistenceModule->PostRestoreObject(Object, RestoredProperties);
		}

		// Some tasks like actor destruction and runtime respawning are deferred until the level is made visible.
		// Handle a special case where the level is already visible when the game code decided to deserialize
		// persistence data.
		if (!InLevel->bIsVisible)
		{
			// Restored state to a level that isn't visible yet. If plugin is configured as such, removed actors can be marked as garbage instead.
			// In that case, instead of beginning and then ending play, they never enter play. This is for offline sessions only, since actors need
			// to enter play and be destroyed on the server for proper net cleanup.
			const ULevelStreamingPersistenceSettings* Settings = GetDefault<ULevelStreamingPersistenceSettings>();
			if (!LevelProperties->RemovedMapActors.IsEmpty() && Settings->CanRemoveActorsBeforeBeginPlay() && GetWorld()->IsNetMode(NM_Standalone))
			{
				ReapplyRemovedActors(LevelProperties, InLevel, /*bMarkAsGarbage=*/true);
			}
		}
		else
		{
			// Special case: We restored properties on a level that was already visible and in play.
			// In that case, we're doing spawning and destruction immediately. This case is hit if
			// the game code decided to deserialize persistence data late.
			DestroyedCount += ReapplyRemovedActors(LevelProperties, InLevel, /*bMarkAsGarbage=*/false);

			// Always respawn here. This branch only runs for already-visible levels (persistent level on actors-initialized,
			// or late InitializeFrom via RestoreVisibleLevels). We won't reach this code path for a reused streaming level: 
			// streaming reuse always goes through OnLevelBeginMakingVisible while the level isn't visible yet. So here we 
			// want to apply the respawn records, whether they were loaded early or late.
			SpawnedCount += RestoreRuntimeSpawnedActors(LevelProperties, InLevel);
		}
	}

	// Finish up restoration now, or defer some work to OnLevelAddedToWorld by enqueueing the level in LevelsPendingFinishRestore
	if (!InLevel->bIsVisible)
	{
		// If the level isn't visible, defer remove, respawn and callback to OnLevelAddedToWorld.
		// If the level is new, this just defers the callback.
		LevelsPendingFinishRestore.Add(InLevel->GetPathName());
	}
	else
	{
		// If the level is visible, fire the callback now. Remove and respawn, if applicable, have been
		// performed above. Regardless of if there was persistent data, fire the callback now. This is
		// primarily for easier usability so that game code can treat PostRestoreLevel as 'initial
		// values have been finalized' regardless of whether anything was loaded.
		PersistenceModule->PostRestoreLevel.ExecuteIfBound(InLevel);

		// Store a weak reference to the level we just finished restoring on. Respawn tasks aren't applied 
		// to this level again when it becomes invisible and visible. It only happens again if the level 
		// gets GCed and reloaded.
		LevelProperties->LastKnownLevel = InLevel;
	}

	return ((RestoredCount + CreatedCount + DestroyedCount + SpawnedCount) > 0);
}

int32 ULevelStreamingPersistenceManager::RestorePrivateProperties(UObject* Object, const FLevelStreamingPersistentObjectPrivateProperties& PersistentObjectProperties, TArray<const FProperty*>& OutRestored) const
{
	int32 RestoredCount = 0;
	check(IsValid(Object));

	// Restore private properties, skipping any that no longer satisfy their configured class filters.
	if (!PersistentObjectProperties.PayloadData.IsEmpty())
	{
		TArray<const FProperty*> FilteredProperties;
		PersistentObjectProperties.ForEachPersistentProperty([this, Object, &FilteredProperties](const FProperty* ObjectProperty)
		{
			if (PersistentPropertiesInfo->SatisfiesFilters(ULevelStreamingPersistentPropertiesInfo::PropertyType_Private, ObjectProperty, Object))
			{
				FilteredProperties.Add(ObjectProperty);
			}
		});

		if (!FilteredProperties.IsEmpty())
		{
			FPersistentPropertiesReader ReaderAr(bUseTaggedPropertySerialization, PersistentObjectProperties.PayloadData);
			FPersistentPropertiesArchive Ar(ReaderAr, FilteredProperties);
			Object->SerializeScriptProperties(Ar);
			for (const FProperty* ObjectProperty : FilteredProperties)
			{
				PersistenceModule->PostRestorePersistedProperty(Object, ObjectProperty);
			}
			OutRestored.Append(FilteredProperties);
			++RestoredCount;
		}
	}
	return RestoredCount;
};

int32 ULevelStreamingPersistenceManager::RestorePublicProperties(UObject* Object, const FLevelStreamingPersistentObjectPublicProperties& ObjectsPublicPropertyValues, TArray<const FProperty*>& OutRestored) const
{
	int32 RestoredCount = 0;
	check(IsValid(Object));
	const UClass* ObjectClass = Object->GetClass();
	// Restore public properties
	if (ObjectsPublicPropertyValues.PropertyBag.IsValid())
	{
		ObjectsPublicPropertyValues.ForEachPersistentProperty([this, Object, ObjectClass, &ObjectsPublicPropertyValues, &RestoredCount, &OutRestored](const FProperty* BagProperty)
		{
			FProperty* ObjectProperty = ObjectClass->FindPropertyByName(BagProperty->GetFName());
			if (!ObjectProperty || !PersistentPropertiesInfo->SatisfiesFilters(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, ObjectProperty, Object))
			{
				return;
			}
			if (CopyPropertyBagValueToObject(&ObjectsPublicPropertyValues.PropertyBag, Object, ObjectProperty))
			{
				OutRestored.Add(ObjectProperty);
				++RestoredCount;
			}
		});
	}
	return RestoredCount;
};

void ULevelStreamingPersistenceManager::BuildRuntimeSpawnedActorRecord(AActor* InActor, FRuntimeSpawnedActorRecord& OutRecord) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULevelStreamingPersistenceManager::BuildRuntimeSpawnedActorRecord);

	OutRecord.ClassPathName = InActor->GetClass()->GetPathName();

	// Pre-serialization hook for the actor to allow it to prepare/update properties to save
	PersistenceModule->PrePersistObject(InActor);

	// When respawning the actor into a streaming level, the transform is treated as relative to that level's transform.
	// Therefore, the spawn record should contain a relative transform.
	const ULevelStreaming* OwningStreamingLevel = FLevelUtils::FindStreamingLevel(InActor->GetLevel());
	const FTransform LevelTransform = OwningStreamingLevel ? OwningStreamingLevel->LevelTransform : FTransform::Identity;
	OutRecord.Transform = InActor->GetActorTransform().GetRelativeTransform(LevelTransform);

	// Private properties: diff against CDO as the baseline
	OutRecord.PrivateProperties.SourceClassPathName = OutRecord.ClassPathName;
	{
		FLevelStreamingPersistentObjectPropertyBag CDOSnapshot;
		if (BuildSnapshot(InActor->GetClass()->GetDefaultObject(), CDOSnapshot))
		{
			if (DiffWithSnapshot(InActor, CDOSnapshot, OutRecord.PrivateProperties.PersistentProperties))
			{
				FPersistentPropertiesWriter WriterAr(bUseTaggedPropertySerialization, OutRecord.PrivateProperties.PayloadData);
				FPersistentPropertiesArchive Ar(WriterAr, OutRecord.PrivateProperties.PersistentProperties);
				InActor->SerializeScriptProperties(Ar);
			}
		}
	}

	// Public properties: copy current values into a property bag
	{
		const UClass* ActorClass = InActor->GetClass();
		if (PersistentPropertiesInfo->HasProperties(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, ActorClass))
		{
			OutRecord.PublicProperties.SourceClassPathName = OutRecord.ClassPathName;
			OutRecord.PublicProperties.PropertyBag.Initialize([this, ActorClass]()
			{
				return PersistentPropertiesInfo->GetPropertyBagFromClass(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, ActorClass);
			});
			PersistentPropertiesInfo->ForEachProperty(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, ActorClass,
				[this, InActor, &OutRecord](FProperty* ObjectProperty)
				{
					if (PersistenceModule->ShouldPersistProperty(InActor, ObjectProperty)
						&& PersistentPropertiesInfo->SatisfiesFilters(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, ObjectProperty, InActor))
					{
						if (const FProperty* BagProp = OutRecord.PublicProperties.PropertyBag.CopyPropertyValueFromObject(InActor, ObjectProperty))
						{
							OutRecord.PublicProperties.PersistentProperties.Add(BagProp);
						}
					}
				});
		}
	}

	// Default subobjects: iterate objects directly outered to InActor that have configured persistent properties
	ForEachObjectWithOuter(InActor, [this, InActor, &OutRecord](UObject* Subobject)
	{
		// Check whether subobject class has potential properties to persist. This is dictated by configured properties,
		// but before applying custom game code conditions (custom ShouldPersistProperty) and outer class filters.
		const UClass* SubobjectClass = Subobject->GetClass();
		const bool bHasPrivate = PersistentPropertiesInfo->HasProperties(ULevelStreamingPersistentPropertiesInfo::PropertyType_Private, SubobjectClass);
		const bool bHasPublic = PersistentPropertiesInfo->HasProperties(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, SubobjectClass);
		if (!bHasPrivate && !bHasPublic)
		{
			return;
		}

		// Fire pre-persist hook for the subobject to allow it to prepare/update properties to save
		PersistenceModule->PrePersistObject(Subobject);

		FRuntimeSpawnedActorSubobjectRecord SubobjectRecord;
		const FString SubobjectClassPathName = SubobjectClass->GetPathName();
		SubobjectRecord.RelativePathName = Subobject->GetPathName(InActor);
		SubobjectRecord.PrivateProperties.SourceClassPathName = SubobjectClassPathName;
		SubobjectRecord.PublicProperties.SourceClassPathName = SubobjectClassPathName;

		// Private properties: diff against the subobject's archetype (the CDO's default subobject)
		if (bHasPrivate)
		{
			if (UObject* Archetype = Subobject->GetArchetype())
			{
				FLevelStreamingPersistentObjectPropertyBag ArchetypeSnapshot;
				if (BuildSnapshot(Archetype, ArchetypeSnapshot))
				{
					if (DiffWithSnapshot(Subobject, ArchetypeSnapshot, SubobjectRecord.PrivateProperties.PersistentProperties))
					{
						FPersistentPropertiesWriter WriterAr(bUseTaggedPropertySerialization, SubobjectRecord.PrivateProperties.PayloadData);
						FPersistentPropertiesArchive Ar(WriterAr, SubobjectRecord.PrivateProperties.PersistentProperties);
						Subobject->SerializeScriptProperties(Ar);
					}
				}
			}
		}

		// Public properties: copy current values into a property bag
		if (bHasPublic)
		{
			SubobjectRecord.PublicProperties.PropertyBag.Initialize([this, SubobjectClass]()
			{
				return PersistentPropertiesInfo->GetPropertyBagFromClass(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, SubobjectClass);
			});
			PersistentPropertiesInfo->ForEachProperty(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, SubobjectClass,
				[this, Subobject, &SubobjectRecord](FProperty* ObjectProperty)
				{
					if (PersistenceModule->ShouldPersistProperty(Subobject, ObjectProperty)
						&& PersistentPropertiesInfo->SatisfiesFilters(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, ObjectProperty, Subobject))
					{
						if (const FProperty* BagProp = SubobjectRecord.PublicProperties.PropertyBag.CopyPropertyValueFromObject(Subobject, ObjectProperty))
						{
							SubobjectRecord.PublicProperties.PersistentProperties.Add(BagProp);
						}
					}
				});
		}

		// Add the record only if something was actually persisted
		if (!SubobjectRecord.PrivateProperties.PersistentProperties.IsEmpty() || !SubobjectRecord.PublicProperties.PersistentProperties.IsEmpty())
		{
			OutRecord.Subobjects.Add(MoveTemp(SubobjectRecord));
		}
	}, EGetObjectsFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage);
}

AActor* ULevelStreamingPersistenceManager::SpawnActorFromRecord(const FRuntimeSpawnedActorRecord& InRecord, ULevel* InLevel) const
{
	// Resolve the class from the spawn record
	FSoftClassPath ClassPath(InRecord.ClassPathName);
	UClass* Class = ClassPath.ResolveClass();
	if (!Class)
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "SpawnActorFromRecord: Class '%ls' failed: the class wasn't resolvable.", *InRecord.ClassPathName);
		return nullptr;
	}

	// Convert the level-relative record transform to world space. At save-time we store the actor transform relative 
	// to its streaming level, to support streaming levels with dynamic level transforms.
	const ULevelStreaming* StreamingLevel = FLevelUtils::FindStreamingLevel(InLevel);
	const FTransform LevelTransform = StreamingLevel ? StreamingLevel->LevelTransform : FTransform::Identity;
	const FTransform WorldTransform = InRecord.Transform * LevelTransform;

	// Spawn the actor deferred, in the streaming level it was recorded in. We spawn deferred so that
	// some values, particularly direct properties and native default subobject values, can be restored
	// before OnConstruction and BeginPlay.
	FActorSpawnParameters SpawnParams;
	SpawnParams.OverrideLevel = InLevel;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnParams.bDeferConstruction = true;
	AActor* NewActor = GetWorld()->SpawnActor(Class, &WorldTransform, SpawnParams);
	if (!IsValid(NewActor))
	{
		return nullptr;
	}

	// Restore properties on the actor itself
	TArray<const FProperty*> RestoredProperties;
	RestorePrivateProperties(NewActor, InRecord.PrivateProperties, RestoredProperties);
	RestorePublicProperties(NewActor, InRecord.PublicProperties, RestoredProperties);

	// Restore subobject records in two passes. Native default subobjects (created in the actor
	// constructor) are available before FinishSpawning; SCS components (added by Blueprint's
	// construction script) are only created during FinishSpawning, so they require a second pass.
	auto RestoreSubobjectRecord = [this, NewActor](const FRuntimeSpawnedActorSubobjectRecord& SubobjectRecord) -> bool
	{
		UObject* Subobject = FindObject<UObject>(NewActor, *SubobjectRecord.RelativePathName);
		if (!IsValid(Subobject))
		{
			return false;
		}

		TArray<const FProperty*> SubobjectRestoredProps;
		RestorePrivateProperties(Subobject, SubobjectRecord.PrivateProperties, SubobjectRestoredProps);
		RestorePublicProperties(Subobject, SubobjectRecord.PublicProperties, SubobjectRestoredProps);
		if (!SubobjectRestoredProps.IsEmpty())
		{
			PersistenceModule->PostRestoreObject(Subobject, SubobjectRestoredProps);
		}
		return true;
	};

	// First pass: native default subobjects. SCS components are not resolvable before FinishSpawning().
	TArray<int32> DeferredSubobjectIndices;
	for (int32 i = 0; i < InRecord.Subobjects.Num(); ++i)
	{
		if (!RestoreSubobjectRecord(InRecord.Subobjects[i]))
		{
			DeferredSubobjectIndices.Add(i);
		}
	}

	// Finish spawning the actor. Internally this constructs SCS components, initializes them and then triggers actor BeginPlay.
	NewActor->FinishSpawning(WorldTransform);

	// Second pass: SCS components, which are only available now.
	for (const int32 i : DeferredSubobjectIndices)
	{
		const FRuntimeSpawnedActorSubobjectRecord& SubobjectRecord = InRecord.Subobjects[i];
		if (!RestoreSubobjectRecord(SubobjectRecord))
		{
			UE_LOGF(LogLevelStreamingPersistence, Verbose, "SpawnActorFromRecord: Could not find subobject '%ls' on respawned actor '%ls'", *SubobjectRecord.RelativePathName, *NewActor->GetName());
		}
	}

	// Fire the actor-level post-restore hook after all properties and subobjects have been restored. This is after its BeginPlay.
	PersistenceModule->PostRestoreObject(NewActor, RestoredProperties);
	return NewActor;
}

void ULevelStreamingPersistenceManager::PreloadRuntimeSpawnedActorClasses()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULevelStreamingPersistenceManager::PreloadRuntimeSpawnedActorClasses);

	for (FSoftClassPath RuntimeRespawnableClass : RuntimeSpawnedActorClassPaths)
	{
		if (RuntimeRespawnableClass.IsValid())
		{
			if (UClass* ActorClass = RuntimeRespawnableClass.TryLoadClass<AActor>())
			{
				PreloadedRuntimeSpawnedActorClasses.Add(ActorClass);
				UE_LOGF(LogLevelStreamingPersistence, Verbose, "Preloaded runtime respawnable actor class '%ls' that was present in serialized data", *RuntimeRespawnableClass.ToString());
			}
			else
			{
				UE_LOGF(LogLevelStreamingPersistence, Warning, "Failed to preload runtime respawnable actor class '%ls' that was present in serialized data", *RuntimeRespawnableClass.ToString());
			}
		}
	}
}

int32 ULevelStreamingPersistenceManager::RestoreRuntimeSpawnedActors(FLevelStreamingPersistentPropertyValues* LevelProperties, ULevel* Level)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULevelStreamingPersistenceManager::RestoreRuntimeSpawnedActors);

	int32 Count = 0;
	const ULevelStreamingPersistenceSettings* Settings = GetDefault<ULevelStreamingPersistenceSettings>();
	for (const FRuntimeSpawnedActorRecord& Record : LevelProperties->RuntimeSpawnedActors)
	{
		TSubclassOf<AActor> Class = FSoftClassPath(Record.ClassPathName).ResolveClass();
		if (!Class)
		{
			UE_LOGF(LogLevelStreamingPersistence, Warning, "RestoreRuntimeSpawnedActors: failed to resolve class '%ls' from spawn record; actor will not be respawned.", *Record.ClassPathName);
			continue;
		}
		if (!Settings->ShouldPersistRuntimeActorClass(Class))
		{
			continue;
		}
		if (SpawnActorFromRecord(Record, Level))
		{
			++Count;
		}
	}
	return Count;
}

FLevelStreamingPersistentObjectPropertyBag* ULevelStreamingPersistenceManager::GetPropertyBag(const FString& InObjectPathName)
{
	FString LevelPathName;
	FString ObjectShortPathName;
	if (!SplitObjectPath(InObjectPathName, LevelPathName, ObjectShortPathName))
	{
		return nullptr;
	}
	FLevelStreamingPersistentPropertyValues* LevelProperties = LevelsPropertyValues.Find(LevelPathName);
	FLevelStreamingPersistentObjectPublicProperties* ObjectPublicPropertyValues = LevelProperties ? LevelProperties->ObjectsPublicPropertyValues.Find(InObjectPathName) : nullptr;
	return ObjectPublicPropertyValues ? &ObjectPublicPropertyValues->PropertyBag : nullptr;
}

const FLevelStreamingPersistentObjectPropertyBag* ULevelStreamingPersistenceManager::GetPropertyBag(const FString& InObjectPathName) const
{
	return const_cast<ULevelStreamingPersistenceManager*>(this)->GetPropertyBag(InObjectPathName);
}

bool ULevelStreamingPersistenceManager::CopyPropertyBagValueToObject(const FLevelStreamingPersistentObjectPropertyBag* InPropertyBag, UObject* InObject, FProperty* InObjectProperty) const
{
	if (InPropertyBag->CopyPropertyValueToObject(InObject, InObjectProperty))
	{
		PersistenceModule->PostRestorePersistedProperty(InObject, InObjectProperty);
		return true;
	}
	return false;
}

TPair<UObject*, FProperty*> ULevelStreamingPersistenceManager::GetObjectPropertyPair(const FString& InObjectPathName, const FName InPropertyName) const
{
	UObject* Object = FindObject<UObject>(nullptr, *InObjectPathName);
	FProperty* ObjectProperty = ::IsValid(Object) ? Object->GetClass()->FindPropertyByName(InPropertyName) : nullptr;
	return TPair<UObject*, FProperty*>(Object, ObjectProperty);
}

const FString ULevelStreamingPersistenceManager::GetResolvedObjectPathName(const FString& InObjectPathName) const
{
	if (UWorldPartition* WorldPartition = GetWorld()->GetWorldPartition())
	{
		const FSoftObjectPath Path(InObjectPathName);
		const FSoftObjectPath WorldPath(Path.GetAssetPath(), FString());
		const UWorld* World = Cast<UWorld>(WorldPath.ResolveObject());
		if (World && (World == GetWorld()))
		{
			FSoftObjectPath ResolvedPath;
			if (FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(FSoftObjectPath(InObjectPathName), ResolvedPath))
			{
				return ResolvedPath.ToString();
			}
		}
	}
	return InObjectPathName;
}

bool ULevelStreamingPersistenceManager::SplitObjectPath(const FString& InObjectPathName, FString& OutLevelPathName, FString& OutShortObjectPathName) const
{
	FSoftObjectPath ObjectPath(InObjectPathName);
	if (ObjectPath.GetSubPathString().StartsWith(TEXT("PersistentLevel.")))
	{
		TStringBuilder<FName::StringBufferSize> Builder;
		Builder << ObjectPath.GetAssetPath() << SUBOBJECT_DELIMITER_CHAR << TEXT("PersistentLevel");
		OutLevelPathName = Builder.ToString();
		OutShortObjectPathName = ObjectPath.GetSubPathString().RightChop(16);
		return true;
	}
	return false;
}


bool ULevelStreamingPersistenceManager::BuildSnapshot(const UObject* InObject, FLevelStreamingPersistentObjectPropertyBag& OutSnapshot) const
{
	int SavedCount = 0;
	const UClass* ObjectClass = InObject->GetClass();
	if (PersistentPropertiesInfo->HasProperties(ULevelStreamingPersistentPropertiesInfo::PropertyType_Private, ObjectClass))
	{
		if (OutSnapshot.Initialize([this, ObjectClass]() { return PersistentPropertiesInfo->GetPropertyBagFromClass(ULevelStreamingPersistentPropertiesInfo::PropertyType_Private, ObjectClass); }))
		{
			PersistentPropertiesInfo->ForEachProperty(ULevelStreamingPersistentPropertiesInfo::PropertyType_Private, ObjectClass, [this, InObject, &SavedCount, &OutSnapshot](FProperty* ObjectProperty)
			{
				if (OutSnapshot.CopyPropertyValueFromObject(InObject, ObjectProperty))
				{
					++SavedCount;
				}
			});
		}
		check(OutSnapshot.IsValid());
	}
	return SavedCount > 0;
}

bool ULevelStreamingPersistenceManager::DiffWithSnapshot(const UObject* InObject, const FLevelStreamingPersistentObjectPropertyBag& InSnapshot, TArray<const FProperty*>& OutChangedProperties) const
{
	if (ensure(InSnapshot.IsValid()))
	{
		const UClass* ObjectClass = InObject->GetClass();

		// Find changed properties
		PersistentPropertiesInfo->ForEachProperty(ULevelStreamingPersistentPropertiesInfo::PropertyType_Private, ObjectClass, [this, InObject, &InSnapshot, &OutChangedProperties](FProperty* ObjectProperty)
		{
			if (PersistenceModule->ShouldPersistProperty(InObject, ObjectProperty)
				&& PersistentPropertiesInfo->SatisfiesFilters(ULevelStreamingPersistentPropertiesInfo::PropertyType_Private, ObjectProperty, InObject))
			{
				bool bIsIdentical = true;
				if (InSnapshot.ComparePropertyValueWithObject(InObject, ObjectProperty, bIsIdentical) && !bIsIdentical)
				{
					OutChangedProperties.Add(ObjectProperty);
				}
			}
		});

		return OutChangedProperties.Num() > 0;
	}

	return false;
}

bool ULevelStreamingPersistenceManager::SerializeTo(TArray<uint8>& OutPayload, const bool bForceUpdate)
{
	// If requested, iterate all visible levels to ensure their latest state is stored, before serializing that.
	if (bForceUpdate)
	{
		UpdateVisibleLevelsPersistentPropertyValues();
	}

	// Serialize to archive and gather custom versions
	TArray<uint8> PayloadData;
	FMemoryWriter PayloadAr(PayloadData, true);
	if (SerializeManager(PayloadAr))
	{
		// Serialize custom versions
		TArray<uint8> HeaderData;
		FMemoryWriter HeaderAr(HeaderData);
		FCustomVersionContainer CustomVersions = PayloadAr.GetCustomVersions();
		CustomVersions.Serialize(HeaderAr);

		// Append data
		OutPayload = MoveTemp(HeaderData);
		OutPayload.Append(PayloadData);
		return true;
	}
	return false;
}

bool ULevelStreamingPersistenceManager::InitializeFrom(const TArray<uint8>& InPayload)
{
	FMemoryReader PayloadAr(InPayload, true);

	// Serialize custom versions
	FCustomVersionContainer CustomVersions;
	CustomVersions.Serialize(PayloadAr);
	PayloadAr.SetCustomVersions(CustomVersions);

	// Serialize payload
	PreloadedRuntimeSpawnedActorClasses.Reset();
	if (SerializeManager(PayloadAr))
	{
		// Preload all runtime spawnable actor classes present in the persistent data to avoid sync loads when respawning.
		// Note: This is currently implemented as a sync load, preferring a one-time hitch on data deserialize over hitches 
		// on potentially multiple points when level streaming. Game code can avoid all hitches here by manually having runtime
		// spawnable actor classes (all runtime respawnable subclasses that may have been recorded during previous session) 
		// already preloaded. I.e: If AMyPickup is runtime respawnable, already have known subclasses BP_PickupA, BP_PickupB 
		// preloaded. Otherwise, they are sync loaded here if previous session recorded them.
		PreloadRuntimeSpawnedActorClasses();

		// If actors are already initialized, apply the loaded persistent data to visible levels right away
		if (GetWorld() && GetWorld()->bActorsInitialized)
		{
			RestoreVisibleLevelsPersistentPropertyValues();
		}
		return true;
	}

	return false;
}

bool ULevelStreamingPersistenceManager::SerializeManager(FArchive& InArchive)
{
	if (!InArchive.IsPersistent())
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "Archive must be persistent to serialize LevelStreamingPersistenceManager");
		return false;
	}

	FObjectAndNameAsStringProxyArchive Ar(InArchive, false);
	Ar.SetIsLoading(InArchive.IsLoading());
	Ar.SetIsSaving(InArchive.IsSaving());
	Ar.SetIsPersistent(true);
	Ar.SetFilterEditorOnly(true);
	Ar.SetIsTextFormat(false);
	Ar.SetWantBinaryPropertySerialization(!bUseTaggedPropertySerialization);
	Ar.UsingCustomVersion(FLevelStreamingPersistenceCustomVersion::GUID);
	Ar.ArShouldSkipBulkData = true;

	bool bLocalUseTaggedPropertySerialization = bUseTaggedPropertySerialization;
	Ar << bLocalUseTaggedPropertySerialization;

	if (bLocalUseTaggedPropertySerialization != bUseTaggedPropertySerialization)
	{
		UE_LOGF(LogLevelStreamingPersistence, Error, "Tagged property serialization mismatch : Can't serialize LevelStreamingPersistenceManager");
		return false;
	}

	if (Ar.IsLoading())
	{
		// Deserialize into temporary struct to allow for cleanup. Only keep what is still relevant.
		TMap<FString, FLevelStreamingPersistentPropertyValues> LocalLevelsPropertyValues;
		Ar << LocalLevelsPropertyValues;

		for (auto& [LocalLevelPathName, LocalLevelPropertyValues] : LocalLevelsPropertyValues)
		{
			// Try to resolve level right away
			ULevel* Level = FindObject<ULevel>(nullptr, *LocalLevelPathName);
			if (IsValid(Level) && Level == GetWorld()->PersistentLevel && !GetDefault<ULevelStreamingPersistenceSettings>()->ShouldIncludePersistentLevel())
			{
				// Skip the persistent level entry if the latest settings disable persistence for it.
				continue;
			}

			FLevelStreamingPersistentPropertyValues& LevelProperties = LevelsPropertyValues.FindOrAdd(LocalLevelPathName);
			for (auto& [ObjectPath, LocalPrivatePropertyValues] : LocalLevelPropertyValues.ObjectsPrivatePropertyValues)
			{
				if (LocalPrivatePropertyValues.Sanitize(*this))
				{
					check(!LocalPrivatePropertyValues.Snapshot.IsValid());
					// Override existing entry with the loaded/sanitized version 
					LevelProperties.ObjectsPrivatePropertyValues.Add(ObjectPath, MoveTemp(LocalPrivatePropertyValues));
					LevelProperties.bIsMakingVisibleCacheValid = false;
				}
			}

			for (auto& [ObjectPath, LocalPublicPropertyValues] : LocalLevelPropertyValues.ObjectsPublicPropertyValues)
			{
				if (LocalPublicPropertyValues.Sanitize(*this))
				{
					check(LocalPublicPropertyValues.PropertiesToPersist.IsEmpty());
					// Override existing entry with the loaded/sanitized version
					LevelProperties.ObjectsPublicPropertyValues.Add(ObjectPath, MoveTemp(LocalPublicPropertyValues));
					LevelProperties.bIsMakingVisibleCacheValid = false;
				}
			}

			// Reset existing respawn records so repeated InitializeFrom calls don't accumulate duplicates.
			LevelProperties.RuntimeSpawnedActors.Reset();
			for (FRuntimeSpawnedActorRecord& Record : LocalLevelPropertyValues.RuntimeSpawnedActors)
			{
				// Sanitize properties: cleanup persistent data that is no longer tracked by the current persistence settings.
				// Continue respawning the actor regardless of whether any of the property records are empty, or end up empty
				// after sanitize, because respawning actors is supported regardless of property data.
				Record.PrivateProperties.Sanitize(*this);
				Record.PublicProperties.Sanitize(*this);
				for (FRuntimeSpawnedActorSubobjectRecord& SubobjectRecord : Record.Subobjects)
				{
					SubobjectRecord.PrivateProperties.Sanitize(*this);
					SubobjectRecord.PublicProperties.Sanitize(*this);
				}

				// Store the sanitized actor spawn record on this manager
				LevelProperties.RuntimeSpawnedActors.Add(MoveTemp(Record));
				LevelProperties.bIsMakingVisibleCacheValid = false;
			}

			for (FName& ActorName : LocalLevelPropertyValues.RemovedMapActors)
			{
				LevelProperties.RemovedMapActors.Add(MoveTemp(ActorName));
			}
		}

		// Deserialize dynamic class depencies: recorded runtime respawnable actor classes across all streaming levels.
		// These will be preloaded if settings are configured to.
		if (Ar.CustomVer(FLevelStreamingPersistenceCustomVersion::GUID) >= FLevelStreamingPersistenceCustomVersion::RuntimeActorPersistence)
		{
			Ar << RuntimeSpawnedActorClassPaths;
		}
	}
	else if (Ar.IsSaving())
	{
		// Gather the set of recorded runtime respawned actor classes across all streaming levels
		RuntimeSpawnedActorClassPaths.Reset();
		for (const auto& [LevelPathName, LevelPropertyValues] : LevelsPropertyValues)
		{
			for (const FRuntimeSpawnedActorRecord& Record : LevelPropertyValues.RuntimeSpawnedActors)
			{
				RuntimeSpawnedActorClassPaths.Add(FSoftClassPath(Record.ClassPathName));
			}
		}

		// Serialize data per streaming level
		Ar << LevelsPropertyValues;

		// Serialize the dynamic class dependencies
		if (Ar.CustomVer(FLevelStreamingPersistenceCustomVersion::GUID) >= FLevelStreamingPersistenceCustomVersion::RuntimeActorPersistence)
		{
			Ar << RuntimeSpawnedActorClassPaths;
		}
	}

	return true;
}

bool ULevelStreamingPersistenceManager::EjectPlacedActor(AActor* InActor)
{
	if (!IsEnabled())
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "EjectPlacedActor: Persistence is disabled.");
		return false;
	}

	if (!IsValid(InActor) || !InActor->IsNetStartupActor())
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "EjectPlacedActor: Actor is invalid or not map-placed.");
		return false;
	}

	ULevel* Level = InActor->GetLevel();
	if (!Level)
	{
		return false;
	}

	// Only eject from levels already tracked by the manager (i.e. that have entered play).
	const FString LevelPathName = Level->GetPathName();
	FLevelStreamingPersistentPropertyValues* LevelProperties = LevelsPropertyValues.Find(LevelPathName);
	if (!LevelProperties)
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "EjectPlacedActor: Level %ls is not tracked by the persistence manager.", *Level->GetName());
		return false;
	}

	// Only eject if destruction persistence is enabled for this actor class.
	if (!GetDefault<ULevelStreamingPersistenceSettings>()->ShouldPersistActorDestruction(InActor))
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "EjectPlacedActor: Actor %ls is not configured for destruction persistence. This is required for ejection.", *InActor->GetName());
		return false;
	}

	LevelProperties->RemovedMapActors.Add(InActor->GetFName());
	UE_LOGF(LogLevelStreamingPersistence, Verbose, "Ejecting map actor %ls: will be removed on reload and picked up by runtime persistence if its class is configured.", *InActor->GetName());
	return true;
}

AActor* ULevelStreamingPersistenceManager::RecreateActorInLevel(AActor* InActor, ULevel* NewOwningLevel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULevelStreamingPersistenceManager::RecreateActorInLevel);

	if (!IsValid(InActor))
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "RecreateActorInLevel: Actor is null or invalid.");
		return nullptr;
	}
	if (!IsValid(NewOwningLevel))
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "RecreateActorInLevel: NewOwningLevel is null or invalid.");
		return nullptr;
	}
	if (InActor->GetLevel() == NewOwningLevel)
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "RecreateActorInLevel: %ls is already in the target level.", *InActor->GetName());

		// Skipping wasteful respawn, but returning original actor to prioritize stability at call-site.
		return InActor;
	}

	const FString NewLevelPathName = NewOwningLevel->GetPathName();
	if (!LevelsPropertyValues.Contains(NewLevelPathName))
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "RecreateActorInLevel: Target level %ls is not indexed for persistence. The recreated actor's state will not be saved on level unload.", *NewOwningLevel->GetName());
	}

	const ULevelStreamingPersistenceSettings* Settings = GetDefault<ULevelStreamingPersistenceSettings>();
	if (InActor->IsNetStartupActor() && !Settings->ShouldPersistActorDestruction(InActor))
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "RecreateActorInLevel: Actor %ls is map-placed but its destruction is not configured to be persistent. This would result in duplicates on streaming level reload. Aborting.", *InActor->GetName());
		return nullptr;
	}

	if (!Settings->ShouldPersistRuntimeActorClass(InActor->GetClass()))
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "RecreateActorInLevel: Actor %ls is not configured as runtime respawnable. Aborting.", *InActor->GetName());
		return nullptr;
	}

	if (!PersistenceModule->ShouldPersistRuntimeActor(InActor))
	{
		UE_LOGF(LogLevelStreamingPersistence, Warning, "RecreateActorInLevel: Actor %ls was rejected by ShouldPersistRuntimeActor game logic. Aborting.", *InActor->GetName());
		return nullptr;
	}

	// Build a record of its class, transform and persistent properties
	FRuntimeSpawnedActorRecord Record;
	BuildRuntimeSpawnedActorRecord(InActor, Record);

	// Convert the actor's relative transform from old level to new level and fixup the spawn record
	auto GetLevelTransform = [](const ULevel* InLevel) -> FTransform
	{
		const ULevelStreaming* StreamingLevel = FLevelUtils::FindStreamingLevel(InLevel);
		return StreamingLevel ? StreamingLevel->LevelTransform : FTransform::Identity;
	};
	const FTransform WorldTransform = Record.Transform * GetLevelTransform(InActor->GetLevel());
	Record.Transform = WorldTransform.GetRelativeTransform(GetLevelTransform(NewOwningLevel));

	// Destroy the current actor
	InActor->Destroy();

	// Recreate it in-place. Up to user to avoid popping or have an acceptable swap.
	return SpawnActorFromRecord(Record, NewOwningLevel);
}

AActor* ULevelStreamingPersistenceManager::RecreateActorInPersistentLevel(AActor* InActor)
{
	return RecreateActorInLevel(InActor, GetWorld()->PersistentLevel);
}

#if !UE_BUILD_SHIPPING
void ULevelStreamingPersistenceManager::DumpContent() const
{
	UE_LOGF(LogLevelStreamingPersistence, Log, "World Persistence Content for %ls", *GetWorld()->GetName());

	TMap<const UClass*, UObject*> TemporaryObjects;
	auto GetTemporaryObjectForClass = [&TemporaryObjects](const UClass* InClass)
	{
		UObject*& Object = TemporaryObjects.FindOrAdd(InClass);
		if (!Object)
		{
			Object = NewObject<UObject>(GetTransientPackage(), InClass);
		}
		return Object;
	};

	for (auto& [LevelPathName, LevelPropertyValues] : LevelsPropertyValues)
	{
		if (LevelPropertyValues.ObjectsPrivatePropertyValues.Num())
		{
			UE_LOGF(LogLevelStreamingPersistence, Log, "[+] Level Private Properties of %ls", *LevelPathName);
			for (const auto& [ObjectPathName, ObjectPropertyValues] : LevelPropertyValues.ObjectsPrivatePropertyValues)
			{
				if (!ObjectPropertyValues.PayloadData.IsEmpty())
				{
					if (const UClass* Class = ObjectPropertyValues.GetSourceClass())
					{
						if (UObject* TempObject = GetTemporaryObjectForClass(Class))
						{
							FString ObjectLevelPathName;
							FString ObjectShortPathName;
							const bool bUseShortName = SplitObjectPath(ObjectPathName, ObjectLevelPathName, ObjectShortPathName);

							UE_LOGF(LogLevelStreamingPersistence, Log, "  [+] Object Private Properties of %ls", bUseShortName ? *ObjectShortPathName : *ObjectPathName);

							TArray<const FProperty*> Dummy;
							if (RestorePrivateProperties(TempObject, ObjectPropertyValues, Dummy))
							{
								for (const FProperty* Property : ObjectPropertyValues.PersistentProperties)
								{
									FString PropertyValue;
									if (PropertyPathHelpers::GetPropertyValueAsString(TempObject, Property->GetName(), PropertyValue))
									{
										UE_LOGF(LogLevelStreamingPersistence, Log, "   - Property[%ls] = %ls", *Property->GetName(), *PropertyValue);
									}
								}
							}
						}
					}
				}
			}
		}

		if (LevelPropertyValues.ObjectsPublicPropertyValues.Num())
		{
			UE_LOGF(LogLevelStreamingPersistence, Log, "[+] Level Public Properties of %ls", *LevelPathName);
			for (auto& [ObjectPathName, ObjectPublicPropertyValues] : LevelPropertyValues.ObjectsPublicPropertyValues)
			{
				if (ObjectPublicPropertyValues.PropertyBag.IsValid())
				{
					FString ObjectLevelPathName;
					FString ObjectShortPathName;
					const bool bUseShortName = SplitObjectPath(ObjectPathName, ObjectLevelPathName, ObjectShortPathName);

					UE_LOGF(LogLevelStreamingPersistence, Log, "  [+] Object Public Properties of %ls", bUseShortName ? *ObjectShortPathName : *ObjectPathName);
					ObjectPublicPropertyValues.PropertyBag.DumpContent([](const FProperty* Property, const FString& PropertyValue)
					{
						UE_LOGF(LogLevelStreamingPersistence, Log, "   - Property[%ls] = %ls", *Property->GetName(), *PropertyValue);
					}, &ObjectPublicPropertyValues.PersistentProperties);
				}
			}
		}
	}
}
#endif

/*
 * FLevelStreamingPersistentObjectPrivateProperties implementation
 */

FLevelStreamingPersistentObjectPrivateProperties::FLevelStreamingPersistentObjectPrivateProperties(FLevelStreamingPersistentObjectPrivateProperties&& InOther)
	: Snapshot(MoveTemp(InOther.Snapshot))
	, SourceClassPathName(MoveTemp(InOther.SourceClassPathName))
	, PayloadData(MoveTemp(InOther.PayloadData))
	, PersistentProperties(InOther.PersistentProperties)
{
}

void FLevelStreamingPersistentObjectPrivateProperties::ForEachPersistentProperty(TFunctionRef<void(const FProperty*)> Func) const
{
	for (const FProperty* Property : PersistentProperties)
	{
		Func(Property);
	}
}

FArchive& operator<<(FArchive& Ar, FLevelStreamingPersistentObjectPrivateProperties& PrivateProperties)
{
	PrivateProperties.Serialize(Ar);
	return Ar;
}

void FLevelStreamingPersistentObjectPrivateProperties::Serialize(FArchive& Ar)
{
	Ar << SourceClassPathName;
	Ar << PayloadData;

	uint32 PersistentPropertiesCount = PersistentProperties.Num();
	Ar << PersistentPropertiesCount;

	const UClass* Class = GetSourceClass();
	UE_CLOGF(!Class, LogLevelStreamingPersistence, Verbose, "FLevelStreamingPersistentObjectPrivateProperties : Could not find class %ls", *SourceClassPathName);

	if (Ar.IsLoading())
	{
		PersistentProperties.Reserve(PersistentPropertiesCount);
		for (uint32 i = 0; i < PersistentPropertiesCount; ++i)
		{
			FString PropertyName;
			Ar << PropertyName;

			if (const FProperty* ObjectProperty = Class ? Class->FindPropertyByName(FName(PropertyName)) : nullptr)
			{
				PersistentProperties.Add(ObjectProperty);
			}
			else
			{
				UE_CLOGF(Class, LogLevelStreamingPersistence, Verbose, "FLevelStreamingPersistentObjectPrivateProperties : Could not find private property %ls", *PropertyName);
			}
		}
	}
	else if (Ar.IsSaving())
	{
		for (const FProperty* Property : PersistentProperties)
		{
			FString PropertyName = Property->GetName();
			Ar << PropertyName;
		}
	}
}

bool FLevelStreamingPersistentObjectPrivateProperties::Sanitize(const ULevelStreamingPersistenceManager& InManager)
{
	check(PersistentProperties.IsEmpty() == PayloadData.IsEmpty());

	if (!PersistentProperties.IsEmpty())
	{
		TArray<const FProperty*> LocalPersistentProperties = MoveTemp(PersistentProperties);
		PersistentProperties.Reserve(LocalPersistentProperties.Num());
		for (const FProperty* PersistentProperty : LocalPersistentProperties)
		{
			if (InManager.PersistentPropertiesInfo->HasProperty(ULevelStreamingPersistentPropertiesInfo::PropertyType_Private, PersistentProperty))
			{
				PersistentProperties.Add(PersistentProperty);
			}
			else if (!InManager.bUseTaggedPropertySerialization)
			{
				UE_LOGF(LogLevelStreamingPersistence, Verbose, "FLevelStreamingPersistentObjectPrivateProperties : ULevelStreamingPersistenceManager doesn't use tagged property serialization and found an invalid persistent property %ls. All private properties for this object will be ignored.", *PersistentProperty->GetName());
				PersistentProperties.Reset();
				return false;
			}
		}
	}
	return !PersistentProperties.IsEmpty();
}

/*
 * FLevelStreamingPersistentObjectPublicProperties implementation
 */

FLevelStreamingPersistentObjectPublicProperties::FLevelStreamingPersistentObjectPublicProperties(FLevelStreamingPersistentObjectPublicProperties&& InOther)
	: PropertiesToPersist(MoveTemp(InOther.PropertiesToPersist))
	, SourceClassPathName(MoveTemp(InOther.SourceClassPathName))
	, PropertyBag(MoveTemp(InOther.PropertyBag))
	, PersistentProperties(InOther.PersistentProperties)

{
}

void FLevelStreamingPersistentObjectPublicProperties::ForEachPersistentProperty(TFunctionRef<void(const FProperty*)> Func) const
{
	for (const FProperty* Property : PersistentProperties)
	{
		Func(Property);
	}
}

FArchive& operator<<(FArchive& Ar, FLevelStreamingPersistentObjectPublicProperties& PublicProperties)
{
	PublicProperties.Serialize(Ar);
	return Ar;
}

void FLevelStreamingPersistentObjectPublicProperties::Serialize(FArchive& Ar)
{
	Ar << SourceClassPathName;
	Ar << PropertyBag;

	uint32 PersistentPropertiesCount = PersistentProperties.Num();
	Ar << PersistentPropertiesCount;
	
	if (Ar.IsLoading())
	{
		PersistentProperties.Reserve(PersistentPropertiesCount);
		for (uint32 i = 0; i < PersistentPropertiesCount; ++i)
		{
			FString PropertyName;
			Ar << PropertyName;
			if (const FProperty* Property = PropertyBag.FindPropertyByName(FName(PropertyName)))
			{
				PersistentProperties.Add(Property);
			}
			else
			{
				UE_LOGF(LogLevelStreamingPersistence, Verbose, "FLevelStreamingPersistentObjectPublicProperties : Could not find public property %ls in property bag.", *PropertyName);
			}
		}
	}
	else if (Ar.IsSaving())
	{
		for (const FProperty* Property : PersistentProperties)
		{
			FString PropertyName = Property->GetName();
			Ar << PropertyName;
		}
	}
}

bool FLevelStreamingPersistentObjectPublicProperties::Sanitize(const ULevelStreamingPersistenceManager& InManager)
{
	if (!PropertyBag.IsValid())
	{
		return false;
	}

	const UClass* SourceClass = GetSourceClass();
	const UPropertyBag* DefaultClass = SourceClass ? InManager.PersistentPropertiesInfo->GetPropertyBagFromClass(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, SourceClass) : nullptr;
	if (!DefaultClass)
	{
		return false;
	}

	FLevelStreamingPersistentObjectPropertyBag LocalPropertyBag = MoveTemp(PropertyBag);
	TArray<const FProperty*> LocalPersistentProperties = MoveTemp(PersistentProperties);
	PropertyBag.Initialize([&InManager, SourceClass]() { return InManager.PersistentPropertiesInfo->GetPropertyBagFromClass(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, SourceClass); });

	for (const FProperty* LocalProperty : LocalPersistentProperties)
	{
		if (const FProperty* Property = PropertyBag.FindPropertyByName(LocalProperty->GetFName()))
		{
			if (PropertyBag.CopyPropertyValueFromPropertyBag(LocalPropertyBag, LocalProperty))
			{
				PersistentProperties.Add(Property);
			}
		}
	}

	if (PersistentProperties.IsEmpty())
	{
		return false;
	}

	return true;
}

/*
 * FRuntimeSpawnedActorSubobjectRecord implementation
 */

void FRuntimeSpawnedActorSubobjectRecord::Serialize(FArchive& Ar)
{
	Ar << RelativePathName;
	Ar << PrivateProperties;
	Ar << PublicProperties;
}

FArchive& operator<<(FArchive& Ar, FRuntimeSpawnedActorSubobjectRecord& Record)
{
	Record.Serialize(Ar);
	return Ar;
}

/*
 * FRuntimeSpawnedActorRecord implementation
 */

void FRuntimeSpawnedActorRecord::Serialize(FArchive& Ar)
{
	Ar << ClassPathName;
	Ar << Transform;
	Ar << PrivateProperties;
	Ar << PublicProperties;
	Ar << Subobjects;
}

FArchive& operator<<(FArchive& Ar, FRuntimeSpawnedActorRecord& Record)
{
	Record.Serialize(Ar);
	return Ar;
}

/*
 * FLevelStreamingPersistentPropertyValues implementation
 */

FArchive& operator<<(FArchive& Ar, FLevelStreamingPersistentPropertyValues& Properties)
{
	Properties.Serialize(Ar);
	return Ar;
}

void FLevelStreamingPersistentPropertyValues::Serialize(FArchive& Ar)
{
	Ar << ObjectsPrivatePropertyValues;
	Ar << ObjectsPublicPropertyValues;
	if (Ar.CustomVer(FLevelStreamingPersistenceCustomVersion::GUID) >= FLevelStreamingPersistenceCustomVersion::RuntimeActorPersistence)
	{
		Ar << RemovedMapActors;
		Ar << RuntimeSpawnedActors;
	}
}

/*
 * ULevelStreamingPersistentPropertiesInfo implementation
 */

bool ULevelStreamingPersistentPropertiesInfo::FPropertyFilters::PassesFilters(const UObject* Object) const
{
	auto MatchesAnyInHierarchy = [](const UClass* Class, const TArray<FSoftClassPath>& FilterPaths)
	{
		for (; Class; Class = Class->GetSuperClass())
		{
			const FSoftClassPath ClassPath(Class);
			for (const FSoftClassPath& FilterPath : FilterPaths)
			{
				if (ClassPath == FilterPath)
				{
					return true;
				}
			}
		}
		return false;
	};

	if (!ObjectClassFilter.IsEmpty() && !MatchesAnyInHierarchy(Object->GetClass(), ObjectClassFilter))
	{
		return false;
	}

	if (!OuterClassFilter.IsEmpty())
	{
		const UObject* Outer = Object->GetOuter();
		const UClass* OuterClass = Outer ? Outer->GetClass() : nullptr;
		if (!OuterClass || !MatchesAnyInHierarchy(OuterClass, OuterClassFilter))
		{
			return false;
		}
	}
	return true;
}

bool ULevelStreamingPersistentPropertiesInfo::SatisfiesFilters(EPropertyType InAccessSpecifier, const FProperty* InProperty, const UObject* InObject) const
{
	if (ensure(InAccessSpecifier < PropertyType_Count))
	{
		if (const FPropertyFilters* Filters = PropertyFiltersMap[InAccessSpecifier].Find(InProperty))
		{
			return Filters->PassesFilters(InObject);
		}
	}
	return true;
}

void ULevelStreamingPersistentPropertiesInfo::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	ULevelStreamingPersistentPropertiesInfo* This = CastChecked<ULevelStreamingPersistentPropertiesInfo>(InThis);
	for (int32 AccessSpecifierIndex = 0; AccessSpecifierIndex < PropertyType_Count; ++AccessSpecifierIndex)
	{
		for (auto& It : This->ClassesProperties[AccessSpecifierIndex])
		{
			Collector.AddReferencedObject(It.Key);
		}
	}
	Super::AddReferencedObjects(InThis, Collector);
}

void ULevelStreamingPersistentPropertiesInfo::Initialize()
{
	auto FillClassesProperties = [this](const FLevelStreamingPersistentProperty& InPersistentProperty) -> bool
	{
		const bool bIsPublic = InPersistentProperty.bIsPublic;
		auto& OutClassesProperties = ClassesProperties[bIsPublic ? PropertyType_Public : PropertyType_Private];
		FSoftObjectPath(InPersistentProperty.Path).TryLoad();
		if (FProperty* Property = TFieldPath<FProperty>(*InPersistentProperty.Path).Get())
		{
			if (const UClass* Class = Property->GetOwnerClass())
			{
				OutClassesProperties.FindOrAdd(Class).Add(Property);

				if (!InPersistentProperty.ObjectClassFilter.IsEmpty() || !InPersistentProperty.OuterClassFilter.IsEmpty())
				{
					FPropertyFilters& Filters = PropertyFiltersMap[bIsPublic ? PropertyType_Public : PropertyType_Private].FindOrAdd(Property);
					Filters.ObjectClassFilter.Append(InPersistentProperty.ObjectClassFilter);
					Filters.OuterClassFilter.Append(InPersistentProperty.OuterClassFilter);
				}

#if WITH_EDITOR
				// Blueprint variables must be marked as instance editable to support persistence, otherwise in PIE the persisted value
				// will be reset to default during RerunConstructionScripts. Only affects PIE which reruns construction scripts at runtime.
				if (Property->HasAnyPropertyFlags(EPropertyFlags::CPF_DisableEditOnInstance) && Cast<UBlueprintGeneratedClass>(Class) != nullptr)
				{
					FText WarningMessage = FText::Format(LOCTEXT("InstanceEditableFormat", "Make variable {0} in {1} Instance Editable to support persistence. "), FText::FromName(Property->GetFName()), FText::FromName(Class->GetFName()));
					FMessageLog MessageLog("LoadErrors");
					MessageLog.Warning(WarningMessage)->AddToken(FUObjectToken::Create(Class->GetPackage()));
					MessageLog.Open(EMessageSeverity::Warning);
				}
#endif
				return true;
			}
		}

		UE_LOGF(LogLevelStreamingPersistence, Log, "Could not resolve property path %ls.", *InPersistentProperty.Path);
		return false;
	};

	auto CreateClassDefaultPropertyBag = [this]()
	{
		for (int32 AccessSpecifierIndex = 0; AccessSpecifierIndex < PropertyType_Count; ++AccessSpecifierIndex)
		{
			EPropertyType AccessSpecifier = EPropertyType(AccessSpecifierIndex);
			TMap<const UClass*, FInstancedPropertyBag>& ClassesDefaults = ObjectClassToPropertyBag[AccessSpecifier];
			for (auto& [Class, ClassProperties] : ClassesProperties[AccessSpecifier])
			{
				check(Class);
				check(!ClassesDefaults.Contains(Class));

				TArray<FPropertyBagPropertyDesc> Descs;
				ForEachProperty(AccessSpecifier, Class, [&Descs](FProperty* Property)
				{
					Descs.Emplace(Property->GetFName(), Property);
#if WITH_EDITORONLY_DATA
					// Get rid of metadata to save memory
					Descs.Last().MetaData.Empty();
#endif
				});

				FInstancedPropertyBag& PropertyBag = ClassesDefaults.FindOrAdd(Class);
				PropertyBag.AddProperties(Descs);
				check(PropertyBag.GetPropertyBagStruct());
			}
		}
	};

	bool bAllPropertiesResolved = true;
	for (const FLevelStreamingPersistentProperty& PersistentProperty : GetDefault<ULevelStreamingPersistenceSettings>()->Properties)
	{
		bAllPropertiesResolved &= FillClassesProperties(PersistentProperty);
	}

#if WITH_EDITOR
	if (!bAllPropertiesResolved)
	{
		// Pop up warning in editor since persistence won't work for some properties. Refer to docs with syntax examples.
		FMessageLog MessageLog("LoadErrors");
		const FString DocsURL = "https://dev.epicgames.com/community/learning/knowledge-base/r6wl/unreal-engine-world-building-guide#wp-importantchangesin53";
		MessageLog.Warning(LOCTEXT("InvalidPaths", "LevelStreamingPersistenceSettings in Engine.ini contained invalid property paths: see log."))
			->AddToken(FURLToken::Create(DocsURL, LOCTEXT("DocsLink", "Documentation: World Building Guide")));
		MessageLog.Open(EMessageSeverity::Warning);
	}
#endif

	CreateClassDefaultPropertyBag();
}

const UPropertyBag* ULevelStreamingPersistentPropertiesInfo::GetPropertyBagFromClass(EPropertyType InAccessSpecifier, const UClass* InClass) const
{
	if (ensure(InAccessSpecifier < PropertyType_Count))
	{
		const UClass* Class = InClass;
		while (Class)
		{
			if (const FInstancedPropertyBag* InstancedPropertyBag = ObjectClassToPropertyBag[InAccessSpecifier].Find(Class))
			{
				return InstancedPropertyBag->GetPropertyBagStruct();
			}
			Class = Class->GetSuperClass();
		}
	}
	return nullptr;
}

bool ULevelStreamingPersistentPropertiesInfo::HasProperty(EPropertyType InAccessSpecifier, const FProperty* InProperty) const
{
	if (ensure(InAccessSpecifier < PropertyType_Count))
	{
		if (const UClass* Class = InProperty->GetOwnerClass())
		{
			if (const TSet<FProperty*>* ClassProperties = ClassesProperties[InAccessSpecifier].Find(Class))
			{
				return ClassProperties->Contains(InProperty);
			}
		}
	}
	return false;
}

void ULevelStreamingPersistentPropertiesInfo::ForEachProperty(EPropertyType InAccessSpecifier, const UClass* InClass, TFunctionRef<void(FProperty*)> Func) const
{
	if (ensure(InAccessSpecifier < PropertyType_Count))
	{
		const UClass* Class = InClass;
		while (Class)
		{
			if (const TSet<FProperty*>* ClassProperties = ClassesProperties[InAccessSpecifier].Find(Class))
			{
				for (FProperty* Property : *ClassProperties)
				{
					Func(Property);
				}
			}
			Class = Class->GetSuperClass();
		}
	}
}

bool ULevelStreamingPersistentPropertiesInfo::HasProperties(EPropertyType InAccessSpecifier, const UClass* InClass) const
{
	if (ensure(InAccessSpecifier < PropertyType_Count))
	{
		const UClass* Class = InClass;
		while (Class)
		{
			if (ClassesProperties[InAccessSpecifier].Contains(Class))
			{
				return true;
			}
			Class = Class->GetSuperClass();
		}
	}
	return false;
}

#if !UE_BUILD_SHIPPING

/*
 * Level Streaming Persistence Manager Console command helper
 */
namespace LSPMConsoleCommandHelper
{
	template<typename PropertyType>
	static bool GetValueFromString(const FString& InPropertyValue, PropertyType& OutResult) { return false; }

	template<typename PropertyType>
	static FString GetValueToString(const PropertyType& InPropertyValue) { return FString(TEXT("<unknown>")); }

	template<typename PropertyType>
	static bool TrySetPropertyValueFromString(ULevelStreamingPersistenceManager* InLevelStreamingPersistenceManager, const FString& InObjectPath, const FName InPropertyName, const FString& InPropertyValue)
	{
		PropertyType ValueFromString;
		if (GetValueFromString(InPropertyValue, ValueFromString))
		{
			if (InLevelStreamingPersistenceManager->TrySetPropertyValue(InObjectPath, InPropertyName, ValueFromString))
			{
				PropertyType Result;
				if (InLevelStreamingPersistenceManager->GetPropertyValue(InObjectPath, InPropertyName, Result))
				{
					FString ResultToString = GetValueToString(Result);
					UE_LOGF(LogLevelStreamingPersistence, Log, "SetPropertyValue succeded : Property[%ls] = %ls for object %ls", *InPropertyName.ToString(), *ResultToString, *InObjectPath);
					return true;
				}
			}
		}
		return false;
	}

	template<typename PropertyType>
	static bool GetPropertyValueAsString(ULevelStreamingPersistenceManager* InLevelStreamingPersistenceManager, const FString& InObjectPath, const FName InPropertyName, FString& OutPropertyValue)
	{
		PropertyType Result;
		if (InLevelStreamingPersistenceManager->GetPropertyValue(InObjectPath, InPropertyName, Result))
		{
			OutPropertyValue = GetValueToString(Result);
			UE_LOGF(LogLevelStreamingPersistence, Log, "GetPropertyValue succeded : Property[%ls] = %ls for object %ls", *InPropertyName.ToString(), *OutPropertyValue, *InObjectPath);
			return true;
		}
		return false;
	}

	template<> bool GetValueFromString(const FString& InPropertyValue, bool& OutResult) { OutResult = InPropertyValue.ToBool(); return true; }
	template<> bool GetValueFromString(const FString& InPropertyValue, int32& OutResult) { LexFromString(OutResult, *InPropertyValue); return true; }
	template<> bool GetValueFromString(const FString& InPropertyValue, int64& OutResult) { LexFromString(OutResult, *InPropertyValue); return true; }
	template<> bool GetValueFromString(const FString& InPropertyValue, float& OutResult) { LexFromString(OutResult, *InPropertyValue); return true; }
	template<> bool GetValueFromString(const FString& InPropertyValue, double& OutResult) { LexFromString(OutResult, *InPropertyValue); return true; }
	template<> bool GetValueFromString(const FString& InPropertyValue, FName& OutResult) { OutResult = FName(InPropertyValue); return true; }
	template<> bool GetValueFromString(const FString& InPropertyValue, FString& OutResult) { OutResult = InPropertyValue; return true; }
	template<> bool GetValueFromString(const FString& InPropertyValue, FText& OutResult) { OutResult = FText::FromString(InPropertyValue); return true; }
	template<> bool GetValueFromString(const FString& InPropertyValue, FVector& OutResult) { return OutResult.InitFromString(InPropertyValue); }
	template<> bool GetValueFromString(const FString& InPropertyValue, FRotator& OutResult) { return OutResult.InitFromString(InPropertyValue); }
	template<> bool GetValueFromString(const FString& InPropertyValue, FTransform& OutResult) { return OutResult.InitFromString(InPropertyValue); }
	template<> bool GetValueFromString(const FString& InPropertyValue, FColor& OutResult) { return OutResult.InitFromString(InPropertyValue); }
	template<> bool GetValueFromString(const FString& InPropertyValue, FLinearColor& OutResult) { return OutResult.InitFromString(InPropertyValue); }

	template<> FString GetValueToString(const bool& InPropertyValue) { return LexToString(InPropertyValue); }
	template<> FString GetValueToString(const int32& InPropertyValue) { return LexToString(InPropertyValue); }
	template<> FString GetValueToString(const int64& InPropertyValue) { return LexToString(InPropertyValue); }
	template<> FString GetValueToString(const float& InPropertyValue) { return LexToString(InPropertyValue); }
	template<> FString GetValueToString(const double& InPropertyValue) { return LexToString(InPropertyValue); }
	template<> FString GetValueToString(const FName& InPropertyValue) { return InPropertyValue.ToString(); }
	template<> FString GetValueToString(const FString& InPropertyValue) { return InPropertyValue; }
	template<> FString GetValueToString(const FText& InPropertyValue) { return InPropertyValue.ToString(); }
	template<> FString GetValueToString(const FVector& InPropertyValue) { return InPropertyValue.ToString(); }
	template<> FString GetValueToString(const FRotator& InPropertyValue) { return InPropertyValue.ToString(); }
	template<> FString GetValueToString(const FTransform& InPropertyValue) { return InPropertyValue.ToString(); }
	template<> FString GetValueToString(const FColor& InPropertyValue) { return InPropertyValue.ToString(); }
	template<> FString GetValueToString(const FLinearColor& InPropertyValue) { return InPropertyValue.ToString(); }

	static FString GetFilename(UWorld* InWorld)
	{
		FString Filename = FPaths::ProjectSavedDir() / TEXT("LevelStreamingPersistence") / FString::Printf(TEXT("LevelStreamingPersistence-%s.bin"), *InWorld->GetName());
		return Filename;
	}

	void ForEachLevelStreamingPersistenceManager(TFunctionRef<bool(ULevelStreamingPersistenceManager*)> Func)
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && World->IsGameWorld())
			{
				if (ULevelStreamingPersistenceManager* LevelStreamingPersistenceManager = World->GetSubsystem<ULevelStreamingPersistenceManager>())
				{
					if (!Func(LevelStreamingPersistenceManager))
					{
						break;
					}
				}
			}
		}
	}

} // namespace LSPMConsoleCommandHelper

FAutoConsoleCommand ULevelStreamingPersistenceManager::DumpContentCommand(
	TEXT("s.LevelStreamingPersistence.Debug.DumpContent"),
	TEXT("Dump the content of LevelStreamingPersistenceManager"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		LSPMConsoleCommandHelper::ForEachLevelStreamingPersistenceManager([](ULevelStreamingPersistenceManager* Manager)
		{
			Manager->DumpContent();
			return true;
		});
	})
);

FAutoConsoleCommand ULevelStreamingPersistenceManager::SetPropertyValueCommand(
	TEXT("s.LevelStreamingPersistence.Debug.SetPropertyValue"),
	TEXT("Set the persistent property's value for a given object"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
	{
		if (InArgs.Num() < 3)
		{
			return;
		}

		LSPMConsoleCommandHelper::ForEachLevelStreamingPersistenceManager([&InArgs](ULevelStreamingPersistenceManager* Manager)
		{
			const FString& ObjectPath = InArgs[0];
			const FName PropertyName = FName(InArgs[1]);
			const FString& PropertyValue = InArgs[2];

			if (LSPMConsoleCommandHelper::TrySetPropertyValueFromString<bool>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::TrySetPropertyValueFromString<int32>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::TrySetPropertyValueFromString<int64>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::TrySetPropertyValueFromString<float>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::TrySetPropertyValueFromString<double>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::TrySetPropertyValueFromString<FName>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::TrySetPropertyValueFromString<FString>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::TrySetPropertyValueFromString<FText>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::TrySetPropertyValueFromString<FVector>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::TrySetPropertyValueFromString<FRotator>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::TrySetPropertyValueFromString<FTransform>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::TrySetPropertyValueFromString<FColor>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::TrySetPropertyValueFromString<FLinearColor>(Manager, ObjectPath, PropertyName, PropertyValue))
			{
				return false;
			}
			else if (Manager->TrySetPropertyValueFromString(ObjectPath, PropertyName, PropertyValue))
			{
				FString Result;
				if (Manager->GetPropertyValueAsString(ObjectPath, PropertyName, Result))
				{
					UE_LOGF(LogLevelStreamingPersistence, Log, "GetPropertyValueAsString succeded : Property[%ls] = %ls for object %ls", *PropertyName.ToString(), *Result, *ObjectPath);
					return false;
				}
			}			
			return true;
		});
	})
);

FAutoConsoleCommand ULevelStreamingPersistenceManager::GetPropertyValueCommand(
	TEXT("s.LevelStreamingPersistence.Debug.GetPropertyValue"),
	TEXT("Get the persistent property's value for a given object"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
	{
		if (InArgs.Num() < 2)
		{
			return;
		}

		LSPMConsoleCommandHelper::ForEachLevelStreamingPersistenceManager([&InArgs](ULevelStreamingPersistenceManager* Manager)
		{
			const FString& ObjectPath = InArgs[0];
			const FName PropertyName = FName(InArgs[1]);

			FString PropertyValue;
			if (LSPMConsoleCommandHelper::GetPropertyValueAsString<bool>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::GetPropertyValueAsString<int32>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::GetPropertyValueAsString<int64>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::GetPropertyValueAsString<float>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::GetPropertyValueAsString<double>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::GetPropertyValueAsString<FName>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::GetPropertyValueAsString<FString>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::GetPropertyValueAsString<FText>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::GetPropertyValueAsString<FVector>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::GetPropertyValueAsString<FRotator>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::GetPropertyValueAsString<FTransform>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::GetPropertyValueAsString<FColor>(Manager, ObjectPath, PropertyName, PropertyValue) ||
				LSPMConsoleCommandHelper::GetPropertyValueAsString<FLinearColor>(Manager, ObjectPath, PropertyName, PropertyValue))
			{
				return false;
			}
			else
			{
				FString Result;
				if (Manager->GetPropertyValueAsString(ObjectPath, PropertyName, Result))
				{
					UE_LOGF(LogLevelStreamingPersistence, Log, "GetPropertyValueAsString succeded : Property[%ls] = %ls for object %ls", *PropertyName.ToString(), *Result, *ObjectPath);
					return false;
				}
			}
			return true;
		});
	})
);

FAutoConsoleCommand ULevelStreamingPersistenceManager::SaveToFileCommand(
	TEXT("s.LevelStreamingPersistence.Debug.SaveToFile"),
	TEXT("Save the content of LevelStreamingPersistenceManager to a file"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		LSPMConsoleCommandHelper::ForEachLevelStreamingPersistenceManager([](ULevelStreamingPersistenceManager* Manager)
		{
			FString Filename = LSPMConsoleCommandHelper::GetFilename(Manager->GetWorld());
			TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*Filename));
			if (FileWriter.IsValid())
			{
				TArray<uint8> Payload;
				if (Manager->SerializeTo(Payload))
				{
					UE_LOGF(LogLevelStreamingPersistence, Log, "SaveToFile %ls succeeded", *Filename);
					*FileWriter << Payload;
					return false;
				}
			}
			return true;
		});
	})
);

FAutoConsoleCommand ULevelStreamingPersistenceManager::LoadFromFileCommand(
	TEXT("s.LevelStreamingPersistence.Debug.LoadFromFile"),
	TEXT("Load from a file and initializes LevelStreamingPersistenceManager"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		LSPMConsoleCommandHelper::ForEachLevelStreamingPersistenceManager([](ULevelStreamingPersistenceManager* Manager)
		{
			FString Filename = LSPMConsoleCommandHelper::GetFilename(Manager->GetWorld());
			TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*Filename));
			if (FileReader.IsValid())
			{
				TArray<uint8> Payload;
				*FileReader << Payload;
				if (Manager->InitializeFrom(Payload))
				{
					UE_LOGF(LogLevelStreamingPersistence, Log, "LoadFromFile %ls succeeded", *Filename);
					return false;
				}
			}
			return true;
		});
	})
);

#endif // !UE_BUILD_SHIPPING

bool ULevelStreamingPersistenceManager::bIsEnabled = true;
FAutoConsoleVariableRef ULevelStreamingPersistenceManager::EnableCommand(
	TEXT("s.LevelStreamingPersistence.Enabled"),
	ULevelStreamingPersistenceManager::bIsEnabled,
	TEXT("Turn on/off to enable/disable world persistent subsystem."),
	ECVF_Default);

#undef LOCTEXT_NAMESPACE
