// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelStreamingPersistenceModule.h"

class FLevelStreamingPersistenceModule : public ILevelStreamingPersistenceModule
{
public:
	virtual void StartupModule() override {};
	virtual void ShutdownModule() override {};
};

IMPLEMENT_MODULE(FLevelStreamingPersistenceModule, LevelStreamingPersistence);