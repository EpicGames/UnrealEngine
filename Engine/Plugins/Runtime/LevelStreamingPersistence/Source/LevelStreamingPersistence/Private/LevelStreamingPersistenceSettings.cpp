// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelStreamingPersistenceSettings.h"
#include "UObject/UnrealType.h"
#include "GameFramework/Actor.h"

ULevelStreamingPersistenceSettings::ULevelStreamingPersistenceSettings()
{
#if WITH_EDITOR
	PropertyOwningClassFilter.Add(AActor::StaticClass()->GetPathName());
	PropertyOwningClassFilter.Add(UActorComponent::StaticClass()->GetPathName());
#endif
}

bool ULevelStreamingPersistenceSettings::ShouldPersistRuntimeActorClass(TSubclassOf<AActor> Class) const
{
	UClass* SuperClass = Class;
	while (SuperClass && SuperClass != UObject::StaticClass())
	{
		if (RuntimeRespawnedActorClasses.Contains(SuperClass))
		{
			return true;
		}
		SuperClass = SuperClass->GetSuperClass();
	}
	return false;
}

bool ULevelStreamingPersistenceSettings::ShouldPersistActorDestruction(const AActor* Actor) const
{
	if (bPersistAllActorDestruction)
	{
		return true;
	}

	UClass* SuperClass = Actor ? Actor->GetClass() : nullptr;
	while (SuperClass && SuperClass != UObject::StaticClass())
	{
		if (PersistedActorDestructionAllowList.Contains(SuperClass))
		{
			return true;
		}
		SuperClass = SuperClass->GetSuperClass();
	}
	return false;
}

#if WITH_EDITOR
TArray<FString> ULevelStreamingPersistenceSettings::ListClassProperties()
{
	TSet<FString> Options;
	for (const FSoftClassPath& ClassPath : PropertyOwningClassFilter)
	{
		if (UClass* Class = ClassPath.TryLoadClass<UObject>())
		{
			for (TFieldIterator<FProperty> PropertyIt(Class, EFieldIterationFlags::None); PropertyIt; ++PropertyIt)
			{
				// Properties that aren't instance editable aren't supported for persistence.
				// For BP variables, users can just mark them instance editable. For native
				// variables, they should be EditAnywhere, EditInstanceOnly, etc.
				if (!PropertyIt->HasAnyPropertyFlags(CPF_DisableEditOnInstance))
				{
					Options.Add(PropertyIt->GetPathName());
				}
			}
		}
	}

	TArray<FString> OptionsSorted = Options.Array();
	OptionsSorted.Sort();
	return OptionsSorted;
}
#endif
