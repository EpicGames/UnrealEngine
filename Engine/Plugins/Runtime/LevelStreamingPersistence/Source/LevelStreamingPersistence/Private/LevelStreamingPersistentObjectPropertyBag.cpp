// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelStreamingPersistentObjectPropertyBag.h"
#include "LevelStreamingPersistenceLog.h"
#include "UObject/PropertyAccessUtil.h"
#include "PropertyPathHelpers.h"

FLevelStreamingPersistentObjectPropertyBag::FLevelStreamingPersistentObjectPropertyBag(FLevelStreamingPersistentObjectPropertyBag&& InOther)
: PropertyBag(MoveTemp(InOther.PropertyBag))
{
}

bool FLevelStreamingPersistentObjectPropertyBag::Initialize(TFunctionRef<const UPropertyBag* ()> InFunc)
{
	if (IsValid())
	{
		return true;
	}
	else
	{
		const UPropertyBag* DefaultClass = InFunc();
		if (ensure(DefaultClass))
		{
			PropertyBag.InitializeFromBagStruct(DefaultClass);
			return true;
		}
	}
	return false;
}

const FProperty* FLevelStreamingPersistentObjectPropertyBag::FindPropertyByName(const FName InPropertyName) const
{
	const FPropertyBagPropertyDesc* Desc = PropertyBag.FindPropertyDescByName(InPropertyName);
	return Desc ? Desc->CachedProperty : nullptr;
}

bool FLevelStreamingPersistentObjectPropertyBag::ComparePropertyValueWithObject(const UObject* InObject, const FProperty* InObjectProperty, bool& bOutIsIdentical) const
{
	check(IsValid());
	bOutIsIdentical = false;
	const FProperty* BagProperty = InObjectProperty ? FindPropertyByName(InObjectProperty->GetFName()) : nullptr;
	if (!BagProperty || !PropertyAccessUtil::ArePropertiesCompatible(BagProperty, InObjectProperty))
	{
		return false;
	}
	const void* ObjectValue = InObjectProperty->ContainerPtrToValuePtr<void>(InObject);
	const void* BagValue = BagProperty->ContainerPtrToValuePtr<void>(PropertyBag.GetValue().GetMemory());
	bOutIsIdentical = PropertyAccessUtil::IsSinglePropertyIdentical(InObjectProperty, ObjectValue, BagProperty, BagValue);
	return true;
}

const FProperty* FLevelStreamingPersistentObjectPropertyBag::GetCompatibleProperty(const FProperty* InObjectProperty) const
{
	check(IsValid());
	const FProperty* BagProperty = InObjectProperty ? FindPropertyByName(InObjectProperty->GetFName()) : nullptr;
	if (!BagProperty)
	{
		return nullptr;
	}

	if (PropertyAccessUtil::ArePropertiesCompatible(BagProperty, InObjectProperty))
	{
		return BagProperty;
	}
	return nullptr;
}

const FProperty* FLevelStreamingPersistentObjectPropertyBag::CopyPropertyValueFromObject(const UObject* InObject, const FProperty* InObjectProperty)
{
	check(IsValid());
	const FProperty* BagProperty = InObjectProperty ? FindPropertyByName(InObjectProperty->GetFName()) : nullptr;
	if (!BagProperty)
	{
		return nullptr;
	}

	const void* BagValue = BagProperty->ContainerPtrToValuePtr<void>(PropertyBag.GetValue().GetMemory());
	EPropertyAccessResultFlags Result = PropertyAccessUtil::GetPropertyValue_Object(InObjectProperty, InObject, BagProperty, const_cast<void*>(BagValue), INDEX_NONE);
	if (Result == EPropertyAccessResultFlags::Success)
	{
		return BagProperty;
	}
	else
	{
		// Requires EditAnywhere, BlueprintReadWrite or some other explicit access specifier to adhere to PropertyAccessUtil rules
		UE_LOGF(LogLevelStreamingPersistence, Warning, "Failed to copy public property '%ls' from object '%ls'. Is the property Instance Editable?", *InObjectProperty->GetName(), *InObject->GetName());
		return nullptr;
	}
}

const FProperty* FLevelStreamingPersistentObjectPropertyBag::CopyPropertyValueFromPropertyBag(const FLevelStreamingPersistentObjectPropertyBag& InSourcePropertyBag, const FProperty* InSourceProperty)
{
	check(IsValid());
	const FProperty* TargetBagProperty = InSourceProperty ? FindPropertyByName(InSourceProperty->GetFName()) : nullptr;
	if (!TargetBagProperty)
	{
		return nullptr;
	}

	const void* TargetBagValue = TargetBagProperty->ContainerPtrToValuePtr<void>(PropertyBag.GetValue().GetMemory());
	EPropertyAccessResultFlags Result = PropertyAccessUtil::GetPropertyValue_InContainer(InSourceProperty, InSourcePropertyBag.PropertyBag.GetValue().GetMemory(), TargetBagProperty, const_cast<void*>(TargetBagValue), INDEX_NONE);
	return (Result == EPropertyAccessResultFlags::Success) ? TargetBagProperty : nullptr;
}

const FProperty* FLevelStreamingPersistentObjectPropertyBag::CopyPropertyValueToObject(UObject* InObject, const FProperty* InObjectProperty) const
{
	check(IsValid());
	const FProperty* BagProperty = ::IsValid(InObject) && InObjectProperty ? FindPropertyByName(InObjectProperty->GetFName()) : nullptr;
	if (!BagProperty)
	{
		return nullptr;
	}
	const void* ObjectValue = InObjectProperty->ContainerPtrToValuePtr<void>(InObject);
	EPropertyAccessResultFlags Result = PropertyAccessUtil::GetPropertyValue_InContainer(BagProperty, PropertyBag.GetValue().GetMemory(), InObjectProperty, const_cast<void*>(ObjectValue), INDEX_NONE);
	return (Result == EPropertyAccessResultFlags::Success) ? BagProperty : nullptr;
}

void FLevelStreamingPersistentObjectPropertyBag::ForEachProperty(TFunctionRef<void(const FProperty*)> Func) const
{
	check(IsValid());
	for (const FPropertyBagPropertyDesc& SourceDesc : PropertyBag.GetPropertyBagStruct()->GetPropertyDescs())
	{
		if (const FProperty* Property = SourceDesc.CachedProperty)
		{
			Func(Property);
		}
	}
}

bool FLevelStreamingPersistentObjectPropertyBag::GetPropertyValueAsString(const FName InPropertyName, FString& OutPropertyValueAsString) const
{
	if (const FProperty* Property = IsValid() ? FindPropertyByName(InPropertyName) : nullptr)
	{
		return PropertyPathHelpers::GetPropertyValueAsString(const_cast<uint8*>(PropertyBag.GetValue().GetMemory()), const_cast<UPropertyBag*>(GetPropertyBagStruct()), InPropertyName.ToString(), OutPropertyValueAsString);
	}
	return false;
}

bool FLevelStreamingPersistentObjectPropertyBag::SetPropertyValueFromString(const FName InPropertyName, const FString& InPropertyValue)
{
	if (const FProperty* Property = IsValid() ? FindPropertyByName(InPropertyName) : nullptr)
	{
		return PropertyPathHelpers::SetPropertyValueFromString(const_cast<uint8*>(PropertyBag.GetValue().GetMemory()), const_cast<UPropertyBag*>(GetPropertyBagStruct()), InPropertyName.ToString(), InPropertyValue);
	}
	return false;
}

void FLevelStreamingPersistentObjectPropertyBag::DumpContent(TFunctionRef<void(const FProperty*, const FString&)> Func, const TArray<const FProperty*>* InDumpProperties)
{
	auto DumpProperty = [this, &Func](const FProperty* Property)
	{
		FString PropertyValue;
		if (GetPropertyValueAsString(Property->GetFName(), PropertyValue))
		{
			Func(Property, *PropertyValue);
		}
	};

	if (InDumpProperties)
	{
		for (const FProperty* Property : *InDumpProperties)
		{
			DumpProperty(Property);
		}
	}
	else
	{
		ForEachProperty([this, &DumpProperty](const FProperty* Property)
		{
			DumpProperty(Property);
		});
	}
}

void FLevelStreamingPersistentObjectPropertyBag::Serialize(FArchive& Ar)
{
	PropertyBag.Serialize(Ar);
}

FArchive& operator<<(FArchive& Ar, FLevelStreamingPersistentObjectPropertyBag& PropertyBag)
{
	PropertyBag.Serialize(Ar);
	return Ar;
}

template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<bool> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Bool; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<int32> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Int32; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<int64> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Int64; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<float> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Float; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<double> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Double; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<FName> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Name; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<FString> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::String; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<FText> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Text; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<FSoftObjectPtr> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::SoftObject; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<UObject*> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Object; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<FVector> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Struct; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<FRotator> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Struct; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<FTransform> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Struct; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<FColor> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Struct; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<FLinearColor> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Struct; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<FStructView> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Struct; };
template<> struct FLevelStreamingPersistentObjectPropertyBag::TPropertyBagPropertyType<FConstStructView> { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::Struct; };

template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const bool& InPropertyValue) { return PropertyBag.SetValueBool(InPropertyName, InPropertyValue); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const int32& InPropertyValue) { return PropertyBag.SetValueInt32(InPropertyName, InPropertyValue); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const int64& InPropertyValue) { return PropertyBag.SetValueInt64(InPropertyName, InPropertyValue); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const float& InPropertyValue) { return PropertyBag.SetValueFloat(InPropertyName, InPropertyValue); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const double& InPropertyValue) { return PropertyBag.SetValueDouble(InPropertyName, InPropertyValue); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const FName& InPropertyValue) { return PropertyBag.SetValueName(InPropertyName, InPropertyValue); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const FString& InPropertyValue) { return PropertyBag.SetValueString(InPropertyName, InPropertyValue); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const FText& InPropertyValue) { return PropertyBag.SetValueText(InPropertyName, InPropertyValue); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const FSoftObjectPtr& InPropertyValue) { return PropertyBag.SetValueObject(InPropertyName, InPropertyValue.Get()); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const UObject& InPropertyValue) { return PropertyBag.SetValueObject(InPropertyName, &InPropertyValue); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const FConstStructView& InPropertyValue) { return PropertyBag.SetValueStruct(InPropertyName, InPropertyValue); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const FVector& InPropertyValue) { return SetValue(InPropertyName, FConstStructView::Make(InPropertyValue)); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const FRotator& InPropertyValue) { return SetValue(InPropertyName, FConstStructView::Make(InPropertyValue)); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const FTransform& InPropertyValue) { return SetValue(InPropertyName, FConstStructView::Make(InPropertyValue)); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const FColor& InPropertyValue) { return SetValue(InPropertyName, FConstStructView::Make(InPropertyValue)); }
template<> EPropertyBagResult FLevelStreamingPersistentObjectPropertyBag::SetValue(const FName InPropertyName, const FLinearColor& InPropertyValue) { return SetValue(InPropertyName, FConstStructView::Make(InPropertyValue)); }

template<> TValueOrError<bool, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return PropertyBag.GetValueBool(InPropertyName); }
template<> TValueOrError<int32, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return PropertyBag.GetValueInt32(InPropertyName); }
template<> TValueOrError<int64, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return PropertyBag.GetValueInt64(InPropertyName); }
template<> TValueOrError<float, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return PropertyBag.GetValueFloat(InPropertyName); }
template<> TValueOrError<double, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return PropertyBag.GetValueDouble(InPropertyName); }
template<> TValueOrError<FName, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return PropertyBag.GetValueName(InPropertyName); }
template<> TValueOrError<FString, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return PropertyBag.GetValueString(InPropertyName); }
template<> TValueOrError<FText, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return PropertyBag.GetValueText(InPropertyName); }
template<> TValueOrError<UObject*, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return PropertyBag.GetValueObject(InPropertyName); }
template<> TValueOrError<FStructView, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return PropertyBag.GetValueStruct(InPropertyName); }
template<> TValueOrError<FVector, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return GetValueStruct<FVector>(InPropertyName); }
template<> TValueOrError<FRotator, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return GetValueStruct<FRotator>(InPropertyName); }
template<> TValueOrError<FTransform, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return GetValueStruct<FTransform>(InPropertyName); }
template<> TValueOrError<FColor, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return GetValueStruct<FColor>(InPropertyName); }
template<> TValueOrError<FLinearColor, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValue(const FName InPropertyName) const { return GetValueStruct<FLinearColor>(InPropertyName); }