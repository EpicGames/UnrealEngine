// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "StructUtils/PropertyBag.h"
#include "Containers/StaticArray.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Subsystems/WorldSubsystem.h"
#include "LevelStreamingPersistentObjectPropertyBag.h"
#include "LevelStreamingPersistenceModule.h"
#include "LevelStreamingPersistenceSettings.h"
#include "LevelStreamingPersistenceManager.generated.h"

class ULevel;
class UWorld;
class ULevelStreaming;
struct FActorsInitializedParams;
using FLevelStreamingPersistentPropertyArray = TArray<FCustomPropertyListNode, TInlineAllocator<32>>;

struct FLevelStreamingPersistentObjectPrivateProperties
{
	FLevelStreamingPersistentObjectPrivateProperties() = default;
	FLevelStreamingPersistentObjectPrivateProperties(FLevelStreamingPersistentObjectPrivateProperties&& Other);

	// Initial snapshot used detect changed properties
	FLevelStreamingPersistentObjectPropertyBag Snapshot;

	// Object source class
	FString SourceClassPathName;
	// Payload data containing serialized changed properties
	TArray<uint8> PayloadData;
	// Persistent Properties
	TArray<const FProperty*> PersistentProperties;

	const UClass* GetSourceClass() const { return LoadObject<UClass>(nullptr, *SourceClassPathName); }
	void ForEachPersistentProperty(TFunctionRef<void(const FProperty*)> Func) const;
	bool Sanitize(const ULevelStreamingPersistenceManager& InManager);
	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FLevelStreamingPersistentObjectPrivateProperties& PrivateProperties);
};

struct FLevelStreamingPersistentObjectPublicProperties
{
	FLevelStreamingPersistentObjectPublicProperties() = default;
	FLevelStreamingPersistentObjectPublicProperties(FLevelStreamingPersistentObjectPublicProperties&& Other);

	// Properties to persist (computed once)
	TArray<const FProperty*> PropertiesToPersist;

	// Object source class
	FString SourceClassPathName;
	// Public properties
	FLevelStreamingPersistentObjectPropertyBag PropertyBag;
	// Persistent Properties
	TArray<const FProperty*> PersistentProperties;

	const UClass* GetSourceClass() const { return LoadObject<UClass>(nullptr, *SourceClassPathName); }
	void ForEachPersistentProperty(TFunctionRef<void(const FProperty*)> Func) const;
	bool Sanitize(const ULevelStreamingPersistenceManager& InManager);
	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FLevelStreamingPersistentObjectPublicProperties& PublicProperties);
};

struct FRuntimeSpawnedActorSubobjectRecord
{
	// Path from subobject to owning actor
	FString RelativePathName;
	// Private properties (diff from subobject archetype)
	FLevelStreamingPersistentObjectPrivateProperties PrivateProperties;
	// Public properties (property bag)
	FLevelStreamingPersistentObjectPublicProperties PublicProperties;

	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRuntimeSpawnedActorSubobjectRecord& Record);
};

struct FRuntimeSpawnedActorRecord
{
	// Class to respawn
	FString ClassPathName;
	// Transform relative to the owning level to spawn at
	FTransform Transform;
	// Private properties (diff from CDO baseline)
	FLevelStreamingPersistentObjectPrivateProperties PrivateProperties;
	// Public properties (property bag)
	FLevelStreamingPersistentObjectPublicProperties PublicProperties;
	// Records of default subobjects outered to this actor that have configured persistent properties
	TArray<FRuntimeSpawnedActorSubobjectRecord> Subobjects;

	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRuntimeSpawnedActorRecord& Record);
};

// Persistence values for a specific level
struct FLevelStreamingPersistentPropertyValues
{
	bool bIsMakingVisibleCacheValid = false;
	// Not serialized. Used to determine whether a level that's being made visible is becoming 
	// visible for the first time or not, i.e. is the level reused, or GCed and newly loaded.
	TWeakObjectPtr<ULevel> LastKnownLevel;
	TMap<FString, FLevelStreamingPersistentObjectPrivateProperties> ObjectsPrivatePropertyValues;
	TMap<FString, FLevelStreamingPersistentObjectPublicProperties> ObjectsPublicPropertyValues;

	// FNames of map-placed actors removed from this level during gameplay. On load, matching actors are destroyed before BeginPlay. 
	// Which actor classes are tracked when destroyed is configured via LevelStreamingPersistenceSettings. Note that RemovedMapActors
	// is built up from multiple events: map actors that were destroyed, and map actors marked for removal via features like 
	// EjectPlacedActor, RecreateActorInPersistentLevel.
	TSet<FName> RemovedMapActors;
	// Records of non-placed actors that were present when the level was saved, and that are configured to be respawnable in LevelStreamingPersistenceSettings.
	TArray<FRuntimeSpawnedActorRecord> RuntimeSpawnedActors;

	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FLevelStreamingPersistentPropertyValues& Properties);
};

// Helper class to access FLevelStreamingPersistentProperty's Properties
UCLASS()
class ULevelStreamingPersistentPropertiesInfo : public UObject
{
	GENERATED_BODY()

public:
	void Initialize();
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	enum EPropertyType
	{
		PropertyType_Public,
		PropertyType_Private,
		PropertyType_Count
	};

	// Per-property object/outer class filters, stored as soft paths to avoid loading classes at init time.
	// Evaluation walks the class hierarchy and compares FSoftClassPath(Class) against each entry.
	struct FPropertyFilters
	{
		// If non-empty: only persist on objects that are, or derive from, any of these classes.
		TArray<FSoftClassPath> ObjectClassFilter;
		// If non-empty: only persist on objects whose Outer is, or derives from, any of these classes.
		TArray<FSoftClassPath> OuterClassFilter;

		bool IsEmpty() const { return ObjectClassFilter.IsEmpty() && OuterClassFilter.IsEmpty(); }
		bool PassesFilters(const UObject* Object) const;
	};

	// Configurable persistent properties helper methods
	const UPropertyBag* GetPropertyBagFromClass(EPropertyType InAccessSpecifier, const UClass* InClass) const;
	void ForEachProperty(EPropertyType InAccessSpecifier, const UClass* InClass, TFunctionRef<void(FProperty*)> Func) const;
	bool HasProperties(EPropertyType InAccessSpecifier, const UClass* InClass) const;
	bool HasProperty(EPropertyType InAccessSpecifier, const FProperty* InProperty) const;

	// Returns whether InObject satisfies the configured ObjectClassFilter and OuterClassFilter for InProperty.
	bool SatisfiesFilters(EPropertyType InAccessSpecifier, const FProperty* InProperty, const UObject* InObject) const;

private:
	/* Acceleration maps to find properties for a given class */
	TStaticArray<TMap<TObjectPtr<const UClass>, TSet<FProperty*>>, PropertyType_Count> ClassesProperties;
	TStaticArray<TMap<const UClass*, FInstancedPropertyBag>, PropertyType_Count> ObjectClassToPropertyBag;
	/* Per-property object and outer class filters */
	TStaticArray<TMap<FProperty*, FPropertyFilters>, PropertyType_Count> PropertyFiltersMap;
};

UCLASS()
class LEVELSTREAMINGPERSISTENCE_API ULevelStreamingPersistenceManager : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	//~ Begin WorldSubsystem
	ULevelStreamingPersistenceManager(const FObjectInitializer&);
	~ULevelStreamingPersistenceManager() {}
	virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End WorldSubsystem

	// Save game utility functions

	// Serialize the persistent properties of all streaming levels of the current map to byte array. This can be put in a SaveGame's property to save to file.
	// @param OutPayload Byte array containing saved values for all streaming levels of the current map, and persistent level if enabled.
	// @param bForceUpdate If true, will iterate all visible streaming levels to update their persistence data. Recommended for saving latest world snapshot.
	UFUNCTION(BlueprintCallable, Category = "Level Streaming Persistence")
	bool SerializeTo(TArray<uint8>& OutPayload, const bool bForceUpdate = false);

	// Deserialize the persistent properties of all streaming levels of the current map from byte array. The payload should be from an earlier SerializeTo call, 
	// from an earlier play session of the same map. Call this only once per map.
	// 
	// It's recommended to call this from a custom UWorldSubsystem::Initialize, early enough to restore properties on persistent level actors before they 
	// BeginPlay, and before initial streaming levels are made visible. You can also call this post-begin play, which is useful for blueprint only projects.
	// Calling this post-begin play will overwrite properties on actors that have begun play.
	// Currently, only native callbacks are provided: PostRestoreObject and PostRestoreLevel.
	// 
	// @param InPayload Byte array containing saved values for all streaming levels of the current map, and persistent level if enabled. 
	// Should be from an earlier SerializeTo call, from the same map but earlier play session. 
	// @return Whether deserialization succeeded
	UFUNCTION(BlueprintCallable, Category = "Level Streaming Persistence")
	bool InitializeFrom(const TArray<uint8>& InPayload);

	// Sets property value and creates the entry if necessary, returns true on success.
	template <typename ClassType, typename PropertyType>
	bool SetPropertyValue(const FString& InObjectPathName, const FName InPropertyName, const PropertyType& InPropertyValue);

	// Sets the property value on existing entries, returns true on success.
	template <typename PropertyType>
	bool TrySetPropertyValue(const FString& InObjectPathName, const FName InPropertyName, const PropertyType& InPropertyValue);

	// Gets the property value if found, returns true on success.
	template<typename PropertyType>
	bool GetPropertyValue(const FString& InObjectPathName, const FName InPropertyName, PropertyType& OutPropertyValue) const;

	// Sets the property value converted from the provided string value on existing entries, returns true on success.
	bool TrySetPropertyValueFromString(const FString& InObjectPathName, const FName InPropertyName, const FString& InPropertyValue);

	// Gets the property value and converts it to a string if found, returns true on success.
	bool GetPropertyValueAsString(const FString& InObjectPathName, const FName InPropertyName, FString& OutPropertyValue);

	// Ejects a map-placed actor: it will be absent on next level reload, but stay alive for the current session.
	// The actor class must be eligible for persistent destruction, see Level Streaming Persistence Settings.
	// Use cases: treat the actor as destroyed while it's alive for animation purposes.
	// 
	// The actor becomes eligible for runtime respawning if it passes your other checks, i.e. RuntimeRespawnedActorClasses
	// and OnShouldPersistRuntimeActor. For handing over an actor for runtime respawning, also see RecreateActorInLevel
	// and RecreateActorInPersistentLevel.
	UFUNCTION(BlueprintCallable, Category = "Level Streaming Persistence")
	bool EjectPlacedActor(AActor* InActor);

	// Captures persistent values for the actor, destroys it and respawns it into another level.
	// Converting a map-placed actor into a runtime respawned one lets it cross boundaries.
	// Deciding the correct method to mitigate visual pop, and restoring references, is left up to the callee.
	// If the actor is map-placed, its class must be configured for persistent destruction.
	// The actor class must be configured to be respawnable.
	//
	// In specific cases such as world already in tear-down, the respawning of the actor may fail while the 
	// original actor is still destroyed.
	UFUNCTION(BlueprintCallable, Category = "Level Streaming Persistence")
	AActor* RecreateActorInLevel(AActor* InActor, ULevel* NewOwningLevel);

	// Calls RecreateActorInLevel passing the persistent level. Used for blueprint exposing the common route of moving an actor 
	// to the persistent level. If the actor is map-placed, its class must be configured for persistent destruction.
	//
	// In specific cases such as world already in tear-down, the respawning of the actor may fail while the 
	// original actor is still destroyed.
	UFUNCTION(BlueprintCallable, Category = "Level Streaming Persistence")
	AActor* RecreateActorInPersistentLevel(AActor* InActor);

protected:
	virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const;

private:
	// Level streaming visibility callbacks
	void OnLevelBeginMakingInvisible(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LoadedLevel);
	void OnLevelBeginMakingVisible(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LoadedLevel);

	// World callback for when a level has been fully added and made visible (actors have had BeginPlay).
	void OnLevelAddedToWorld(ULevel* Level, UWorld* World);

	// World callback for when the persistent levels actors are initialized, so persistent properties can be restored on them. Fires before the actors receive BeginPlay.
	void OnWorldActorsInitialized(const FActorsInitializedParams& Params);
	// World callback for when an actor is destroyed, to track runtime-destroyed actors for persistence.
	void OnActorDestroyed(AActor* DestroyedActor);
	// Redestroys actors in a level that were marked as removed when the level was previously visible. Will call Destroy on actors in play or MarkAsGarbage for actors outside play.
	int32 ReapplyRemovedActors(FLevelStreamingPersistentPropertyValues* LevelProperties, ULevel* InLevel, const bool bMarkAsGarbage);

	// Runtime-respawned actor persistence
	// Makes a record of an actor that will be runtime respawned when the owning level gets reloaded. The record contains class, transform, and records for properties and subobjects.
	void BuildRuntimeSpawnedActorRecord(AActor* InActor, FRuntimeSpawnedActorRecord& OutRecord) const;
	// Respawns an actor from an actor record.
	AActor* SpawnActorFromRecord(const FRuntimeSpawnedActorRecord& InRecord, ULevel* InLevel) const;
	// Preloads a list of actor classes for which runtimed spawnable actor records have been deserialized.
	void PreloadRuntimeSpawnedActorClasses();
	// Respawns all actors from actor records on a level properties entry.
	int32 RestoreRuntimeSpawnedActors(FLevelStreamingPersistentPropertyValues* LevelProperties, ULevel* InLevel);

	// Iterate all currently visible streaming levels to record the persistent properties of objects. Normally, these values are only updated 
	// for a level when it is made invisible. Call this to make sure that the latest gameplay state of all streaming levels, visible and invisible, 
	// is stored in the manager. You can then serialize this data, for example for save game purposes, by calling SerializeTo.
	void UpdateVisibleLevelsPersistentPropertyValues();
	void RestoreVisibleLevelsPersistentPropertyValues();

	// Save persistent properties
	bool SaveLevelPersistentPropertyValues(const ULevel* InLevel);

	// Restore persistent properties
	bool RestoreLevelPersistentPropertyValues(ULevel* InLevel);
	int32 RestorePrivateProperties(UObject* InObject, const FLevelStreamingPersistentObjectPrivateProperties& InPersistentProperties, TArray<const FProperty*>& OutRestored) const;
	int32 RestorePublicProperties(UObject* InObject, const FLevelStreamingPersistentObjectPublicProperties& InPersistentProperties, TArray<const FProperty*>& OutRestored) const;

	// Snapshot of persistent properties
	bool BuildSnapshot(const UObject* InObject, FLevelStreamingPersistentObjectPropertyBag& OutSnapshot) const;
	bool DiffWithSnapshot(const UObject* InObject, const FLevelStreamingPersistentObjectPropertyBag& InSnapshot, TArray<const FProperty*>& OutChangedProperties) const;

	bool IsEnabled() const { return bIsEnabled; }
	bool SplitObjectPath(const FString& InObjectPathName, FString& OutLevelPathName, FString& OutShortObjectPathName) const;
	const FString GetResolvedObjectPathName(const FString& InObjectPathName) const;
	bool CopyPropertyBagValueToObject(const FLevelStreamingPersistentObjectPropertyBag* InPropertyBag, UObject* InObject, FProperty* InObjectProperty) const;
	TPair<UObject*, FProperty*> GetObjectPropertyPair(const FString& InObjectPathName, const FName InPropertyName) const;
#if !UE_BUILD_SHIPPING
	void DumpContent() const;
#endif
	const FLevelStreamingPersistentObjectPropertyBag* GetPropertyBag(const FString& InObjectPathName) const;
	FLevelStreamingPersistentObjectPropertyBag* GetPropertyBag(const FString& InObjectPathName);
	bool SerializeManager(FArchive& Ar);

	template <typename ClassType>
	FLevelStreamingPersistentObjectPropertyBag* GetOrCreatePropertyBag(const FString& InObjectPathName, const FName InPropertyName);

	// Console commands
	static class FAutoConsoleVariableRef EnableCommand;
	static bool bIsEnabled;
#if !UE_BUILD_SHIPPING
	static class FAutoConsoleCommand SaveToFileCommand;
	static class FAutoConsoleCommand LoadFromFileCommand;
	static class FAutoConsoleCommand DumpContentCommand;
	static class FAutoConsoleCommand SetPropertyValueCommand;
	static class FAutoConsoleCommand GetPropertyValueCommand;
#endif

	// Per-level Persistent property values
	mutable TMap<FString, FLevelStreamingPersistentPropertyValues> LevelsPropertyValues;

	// Level paths that have pending destroys and respawns when made visible.
	TSet<FString> LevelsPendingFinishRestore;

	// Set of all class paths required to respawn runtime-spawned actors, populated on save and restored on load.
	// This set is populated regardless of the setting whether to preload or not.
	TSet<FSoftClassPath> RuntimeSpawnedActorClassPaths;
	// Classes kept in memory to ensure they are available for runtime respawning, to avoid sync loads when making streaming levels visible.
	UPROPERTY(Transient)
	TArray<TObjectPtr<UClass>> PreloadedRuntimeSpawnedActorClasses;

	// Persistence Module
	ILevelStreamingPersistenceModule* PersistenceModule = nullptr;

	FDelegateHandle ActorDestroyedHandle;

	// Persistent Properties Info
	UPROPERTY(Transient)
	TObjectPtr<ULevelStreamingPersistentPropertiesInfo> PersistentPropertiesInfo;

	// Whether to use tagget property serialization
	bool bUseTaggedPropertySerialization;

	friend struct FLevelStreamingPersistentObjectPrivateProperties;
	friend struct FLevelStreamingPersistentObjectPublicProperties;
	friend class FPersistenceModule;
};

template <typename PropertyType>
bool ULevelStreamingPersistenceManager::TrySetPropertyValue(const FString& InObjectPathName, const FName InPropertyName, const PropertyType& InPropertyValue)
{
	const FString ObjectPathName = GetResolvedObjectPathName(InObjectPathName);
	FLevelStreamingPersistentObjectPropertyBag* PropertyBag = GetPropertyBag(ObjectPathName);
	if (PropertyBag && PropertyBag->SetPropertyValue(InPropertyName, InPropertyValue))
	{
		// If object is loaded, copy property value to object
		auto ObjectPropertyPair = GetObjectPropertyPair(ObjectPathName, InPropertyName);
		CopyPropertyBagValueToObject(PropertyBag, ObjectPropertyPair.Key, ObjectPropertyPair.Value);
		return true;
	}
	return false;
}

template <typename ClassType, typename PropertyType>
bool ULevelStreamingPersistenceManager::SetPropertyValue(const FString& InObjectPathName, const FName InPropertyName, const PropertyType& InPropertyValue)
{
	const FString ObjectPathName = GetResolvedObjectPathName(InObjectPathName);
	FLevelStreamingPersistentObjectPropertyBag* PropertyBag = GetOrCreatePropertyBag<ClassType>(ObjectPathName, InPropertyName);
	if (PropertyBag && PropertyBag->SetPropertyValue(InPropertyName, InPropertyValue))
	{
		// If object is loaded, copy property value to object
		auto ObjectPropertyPair = GetObjectPropertyPair(ObjectPathName, InPropertyName);
		CopyPropertyBagValueToObject(PropertyBag, ObjectPropertyPair.Key, ObjectPropertyPair.Value);
		return true;
	}
	return false;
}

template<typename PropertyType>
bool ULevelStreamingPersistenceManager::GetPropertyValue(const FString& InObjectPathName, const FName InPropertyName, PropertyType& OutPropertyValue) const
{
	const FString ObjectPathName = GetResolvedObjectPathName(InObjectPathName);
	const FLevelStreamingPersistentObjectPropertyBag* PropertyBag = GetPropertyBag(ObjectPathName);
	return PropertyBag ? PropertyBag->GetPropertyValue(InPropertyName, OutPropertyValue) : false;
}

template <typename ClassType>
FLevelStreamingPersistentObjectPropertyBag* ULevelStreamingPersistenceManager::GetOrCreatePropertyBag(const FString& InObjectPathName, const FName InPropertyName)
{
	const FString ObjectPathName = GetResolvedObjectPathName(InObjectPathName);
	const UClass* ObjectClass = ClassType::StaticClass();

	FString LevelPathName;
	FString ObjectShortPathName;
	if (!SplitObjectPath(ObjectPathName, LevelPathName, ObjectShortPathName))
	{
		return nullptr;
	}

	if (FProperty* ObjectProperty = ObjectClass->FindPropertyByName(InPropertyName))
	{
		if (PersistentPropertiesInfo->HasProperty(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, ObjectProperty))
		{
			FLevelStreamingPersistentPropertyValues& LevelProperties = LevelsPropertyValues.FindOrAdd(LevelPathName);

			FLevelStreamingPersistentObjectPublicProperties& ObjectPublicPropertyValues = LevelProperties.ObjectsPublicPropertyValues.FindOrAdd(ObjectPathName);
			if (ObjectPublicPropertyValues.PropertyBag.Initialize([this, ObjectClass]() { return PersistentPropertiesInfo->GetPropertyBagFromClass(ULevelStreamingPersistentPropertiesInfo::PropertyType_Public, ObjectClass); }))
			{
				return &ObjectPublicPropertyValues.PropertyBag;
			}
		}
	}
	return nullptr;
}
