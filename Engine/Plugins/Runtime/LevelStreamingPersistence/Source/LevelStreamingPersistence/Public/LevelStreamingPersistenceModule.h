// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/Class.h"
#include "GameFramework/Actor.h"

class ULevel;

/**
 * The public interface to Level Streaming Persistence Module.
 * Contains hooks for native game code to override, to implement custom persistence conditions, and pre-save/post-restore tasks.
 */
class LEVELSTREAMINGPERSISTENCE_API ILevelStreamingPersistenceModule : public IModuleInterface
{
public:
	// Delegate for custom conditions per object class for whether a property should be saved.
	// These conditions have to be met in addition to the property being configured in project settings, 
	// and passing the configuration's ObjectClassFilter and OuterClassFilter. 
	// Example use case: Only persist USceneComponent::RelativeLocation on root components.
	// See: OnShouldPersistProperty<ClassType> and OnShouldPersistProperty(Class).
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FShouldPersistProperty, const UObject*, const FProperty*);
	// Delegate for game logic per object class to run after a property value is restored.
	// See: OnPostRestorePersistedProperty<ClassType> and OnPostRestorePersistedProperty(Class).
	DECLARE_DELEGATE_RetVal_TwoParams(void, FPostRestorePersistedProperty, const UObject*, const FProperty*);
	
	// Delegate for game logic per object class that is ran before the object is saved.
	// Use this to do any custom preparation of serializable properties, like inventory to struct.
	// See: OnPrePersistObject<ClassType> and OnPrePersistObject(Class).
	DECLARE_DELEGATE_OneParam(FPrePersistObject, const UObject*);

	// Delegate for game logic per object class that is ran after the object's properties are restored.
	// Use this to do any custom post-restore logic from serializable properties, like struct to runtime inventory.
	// See: OnPostRestoreObject<ClassType> and OnPostRestoreObject(Class).
	DECLARE_DELEGATE_TwoParams(FPostRestoreObject, const UObject*, const TArray<const FProperty*>&);

	// Delegate for game logic to run before a level is saved. Use this to do any preparation of serializable
	// properties that relies on scope beyond single objects. For example: data to restore actor cross-references,
	// copying world subsystem and game state data to persistent level actor, etc.
	// See: PrePersistLevel.
	DECLARE_DELEGATE_OneParam(FPrePersistLevel, const ULevel*);
	// Delegate for game logic to run after a level's state has been restored, so all the properties on objects
	// within that level has been restored, and saved runtime actors have been respawned. As a usability detail,
	// this delegate is also called when a level is first encountered and didn't have persisted data. This allows
	// game code to treat this moment as 'initial values for level are finalized' (restored or don't need restore).
	// See: PostRestoreLevel.
	DECLARE_DELEGATE_OneParam(FPostRestoreLevel, const ULevel*);

	// Delegate for custom conditions per object class for whether a runtime actor should be saved.
	// The actor has to pass this function in addition to the plugin being configured to treat
	// that actor class as runtime respawnable.
	// Use case: Skip actors that are in their (game specific) death animation
	// See: OnShouldPersistRuntimeActor<ActorClass> and OnShouldPersistRuntimeActor(ActorClass).
	DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldPersistRuntimeActor, const AActor*);

	// Game code hook for whether an object's property should be persisted. Is called in addition to ObjectClassFilter and OuterClassFilter.
	template<typename ClassType>
	FShouldPersistProperty& OnShouldPersistProperty()
	{
		return OnShouldPersistProperty(ClassType::StaticClass());
	}

	// Game code hook for whether an object's property should be persisted. Is called in addition to ObjectClassFilter and OuterClassFilter.
	FShouldPersistProperty& OnShouldPersistProperty(const UClass* InClass)
	{
		return ClassShouldPersistProperty.FindOrAdd(InClass);
	}

	// Game code hook for whether a runtime-spawned actor should be recorded for persistence. Is called in addition to RuntimeRespawnedActorClasses settings.
	template<typename ClassType>
	FShouldPersistRuntimeActor& OnShouldPersistRuntimeActor()
	{
		return OnShouldPersistRuntimeActor(ClassType::StaticClass());
	}

	// Game code hook for whether a runtime-spawned actor should be recorded for persistence. Is called in addition to RuntimeRespawnedActorClasses settings.
	FShouldPersistRuntimeActor& OnShouldPersistRuntimeActor(const UClass* InClass)
	{
		return ClassShouldPersistRuntimeActor.FindOrAdd(InClass);
	}

	// Game code hook for after an object's property has been restored.
	template<typename ClassType>
	FPostRestorePersistedProperty& OnPostRestorePersistedProperty()
	{
		return OnPostRestorePersistedProperty(ClassType::StaticClass());
	}

	// Game code hook for after an object's property has been restored.
	FPostRestorePersistedProperty& OnPostRestorePersistedProperty(const UClass* InClass)
	{
		return ClassPostRestorePersistedProperty.FindOrAdd(InClass);
	}

	// Game code hook for before an object will be persisted. Gives an opportunity to do last second conversion of runtime state to serializable properties.
	template<typename ClassType>
	FPrePersistObject& OnPrePersistObject()
	{
		return OnPrePersistObject(ClassType::StaticClass());
	}

	// Game code hook for before an object will be persisted. Gives an opportunity to do last second conversion of runtime state to serializable properties.
	FPrePersistObject& OnPrePersistObject(const UClass* InClass)
	{
		return ClassPrePersistObject.FindOrAdd(InClass);
	}

	// Game code hook for after an object's properties are restored. Gives an opportunity to do custom initialization that depends on the restored properties.
	// For map-placed actors and subobjects, if persistence data is initialized on map load the callback is fired before they BeginPlay. For runtime spawned
	// actors, the callback is fired before BeginPlay on native default subobjects, but after BeginPlay on the actor itself and blueprint added (SCS) components.
	template<typename ClassType>
	FPostRestoreObject& OnPostRestoreObject()
	{
		return OnPostRestoreObject(ClassType::StaticClass());
	}

	// Game code hook for after an object's properties are restored. Gives an opportunity to do custom initialization that depends on the restored properties.
	// For map-placed actors and subobjects, if persistence data is initialized on map load the callback is fired before they BeginPlay. For runtime spawned
	// actors, the callback is fired before BeginPlay on native default subobjects, but after BeginPlay on the actor itself and blueprint added (SCS) components.
	FPostRestoreObject& OnPostRestoreObject(const UClass* InClass)
	{
		return ClassPostRestoreObject.FindOrAdd(InClass);
	}

	// Level-wide hook for before a streaming level will be persisted.
	FPrePersistLevel PrePersistLevel;
	// Level-wide hook for after a streaming level's state is restored. Gives an opportunity to restore inter-actor relations, after their individual state has been restored.
	FPostRestoreLevel PostRestoreLevel;

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline ILevelStreamingPersistenceModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ILevelStreamingPersistenceModule>("LevelStreamingPersistence");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("LevelStreamingPersistence");
	}

private:

	bool ShouldPersistProperty(const UObject* InObject, const FProperty* InProperty) const
	{
		const UClass* Class = InObject->GetClass();
		while (Class)
		{
			const FShouldPersistProperty* Found = ClassShouldPersistProperty.Find(Class);
			if (Found && Found->IsBound() && !Found->Execute(InObject, InProperty))
			{
				return false;
			}
			Class = Class->GetSuperClass();
		}
		return true;
	}

	bool ShouldPersistRuntimeActor(const AActor* InActor) const
	{
		const UClass* Class = InActor->GetClass();
		while (Class)
		{
			const FShouldPersistRuntimeActor* Found = ClassShouldPersistRuntimeActor.Find(Class);
			if (Found && Found->IsBound() && !Found->Execute(InActor))
			{
				return false;
			}
			Class = Class->GetSuperClass();
		}
		return true;
	}

	void PostRestorePersistedProperty(const UObject* InObject, const FProperty* InProperty) const
	{
		const UClass* Class = InObject->GetClass();
		while (Class)
		{
			if (const FPostRestorePersistedProperty* Found = ClassPostRestorePersistedProperty.Find(Class))
			{
				Found->ExecuteIfBound(InObject, InProperty);
			}
			Class = Class->GetSuperClass();
		}
	}
	
	TMap<TWeakObjectPtr<const UClass>, FShouldPersistProperty> ClassShouldPersistProperty;
	TMap<TWeakObjectPtr<const UClass>, FPostRestorePersistedProperty> ClassPostRestorePersistedProperty;
	TMap<TWeakObjectPtr<const UClass>, FShouldPersistRuntimeActor> ClassShouldPersistRuntimeActor;

	void PrePersistObject(const UObject* InObject) const
	{
		const UClass* Class = InObject->GetClass();
		while (Class)
		{
			if (const FPrePersistObject* Found = ClassPrePersistObject.Find(Class))
			{
				Found->ExecuteIfBound(InObject);
			}
			Class = Class->GetSuperClass();
		}
	}

	void PostRestoreObject(const UObject* InObject, const TArray<const FProperty*>& RestoredProperties) const
	{
		const UClass* Class = InObject->GetClass();
		while (Class)
		{
			if (const FPostRestoreObject* Found = ClassPostRestoreObject.Find(Class))
			{
				Found->ExecuteIfBound(InObject, RestoredProperties);
			}
			Class = Class->GetSuperClass();
		}
	}

	TMap<TWeakObjectPtr<const UClass>, FPrePersistObject> ClassPrePersistObject;
	TMap<TWeakObjectPtr<const UClass>, FPostRestoreObject> ClassPostRestoreObject;

	friend class ULevelStreamingPersistenceManager;
};