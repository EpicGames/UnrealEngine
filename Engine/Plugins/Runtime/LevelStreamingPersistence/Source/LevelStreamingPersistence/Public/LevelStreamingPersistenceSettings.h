// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "Templates/SubclassOf.h"
#include "UObject/SoftObjectPath.h"
#include "LevelStreamingPersistenceSettings.generated.h"

class AActor;

USTRUCT()
struct LEVELSTREAMINGPERSISTENCE_API FLevelStreamingPersistentProperty
{
	GENERATED_BODY()

	// Name of the class's property. The property must be a direct member of the class and can't be nested.
	// The property should be instance editable to support persistence.
	UPROPERTY(EditAnywhere, Category = Persistence, meta = (GetOptions = "ListClassProperties"))
	FString Path;

	// Public: Accessible even when unloaded (if initialized).
	UPROPERTY(EditAnywhere, Category = Persistence)
	bool bIsPublic = false;

	// If non-empty: this property will only be persisted on objects of any of these base classes. These can only be subclasses of the property's owner class.
	// Example use case: only persist SceneComponent::RelativeLocation on UMySceneComponent.
	// For more control, bind to ILevelStreamingPersistenceModule::Get().OnShouldPersistProperty natively.
	UPROPERTY(EditAnywhere, Category = Persistence, meta = (AllowedClasses = "/Script/Engine.Actor, /Script/Engine.ActorComponent", AllowAbstract = "true"))
	TArray<FSoftClassPath> ObjectClassFilter;
	// If non-empty: this property will only be persisted on objects with outers of any of these base classes.
	// Example use case: only persist SceneComponent::RelativeLocation on AMyActor.
	// For more control, bind to ILevelStreamingPersistenceModule::Get().OnShouldPersistProperty natively.
	UPROPERTY(EditAnywhere, Category = Persistence, meta = (AllowedClasses = "/Script/Engine.Actor, /Script/Engine.ActorComponent", AllowAbstract = "true"))
	TArray<FSoftClassPath> OuterClassFilter;
};

UCLASS(Config = Engine, DefaultConfig, meta=(DisplayName="Level Streaming Persistence"))
class LEVELSTREAMINGPERSISTENCE_API ULevelStreamingPersistenceSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	ULevelStreamingPersistenceSettings();

	FORCEINLINE bool ShouldRuntimeRespawnAnyActor() const
	{
		return !RuntimeRespawnedActorClasses.IsEmpty();
	}

	bool ShouldPersistRuntimeActorClass(TSubclassOf<AActor> Class) const;

	FORCEINLINE bool ShouldPersistAnyActorDestruction() const
	{
		return bPersistAllActorDestruction || !PersistedActorDestructionAllowList.IsEmpty();
	}

	FORCEINLINE bool ShouldIncludePersistentLevel() const { return bIncludePersistentLevel; }
	FORCEINLINE bool CanRemoveActorsBeforeBeginPlay() const { return bCanRemoveActorsBeforeBeginPlay; }

	bool ShouldPersistActorDestruction(const AActor* Actor) const;

private:
	// If true, persistence data is also saved for actors in the persistent level. Off by default for backwards compatibility.
	UPROPERTY(Config, EditAnywhere, Category = "Persistent Level")
	bool bIncludePersistentLevel = false;

	// Actor base classes to scan for at level-save time. Runtime-spawned instances of these classes
	// will be recorded and respawned with their properties restored when the level reloads.
	UPROPERTY(Config, EditAnywhere, Category = "Runtime Spawned Actors", meta = (AllowedClasses = "/Script/Engine.Actor"))
	TArray<FSoftClassPath> RuntimeRespawnedActorClasses;

	// If true, destruction of map actors of any class will be saved, to be destroyed on level reload.
	UPROPERTY(Config, EditAnywhere, Category = "Destroyed Actors")
	bool bPersistAllActorDestruction = false;
	// If bPersistAllActorDestruction == false: specific actor base classes for map actors whose destruction is saved, to be redestroyed on level reload.
	UPROPERTY(Config, EditAnywhere, Category = "Destroyed Actors", meta = (AllowedClasses = "/Script/Engine.Actor", EditCondition = "!bPersistAllActorDestruction"))
	TArray<FSoftClassPath> PersistedActorDestructionAllowList;
	// If true, actors can be removed before BeginPlay by marking them as garbage before the level enters play. This can save performance in offline play
	// by avoid the actor beginning, then ending play. In online play, actors are always destroyed after BeginPlay to allow proper net cleanup.
	UPROPERTY(Config, EditAnywhere, Category = "Destroyed Actors")
	bool bCanRemoveActorsBeforeBeginPlay = false;

#if WITH_EDITOR
	// Populate property dropdown for entries in 'Properties'
	UFUNCTION()
	TArray<FString> ListClassProperties();
#endif

#if WITH_EDITORONLY_DATA
	// Editor-only: These classes are used to populate the properties dropdown below.
	UPROPERTY(Config, EditAnywhere, Category = "Persisted Properties", meta = (AllowedClasses = "/Script/Engine.Actor, /Script/Engine.ActorComponent", AllowAbstract = "true"))
	TSet<FSoftClassPath> PropertyOwningClassFilter;
#endif

	// Persistence settings per property. Properties are populated based on PropertyOwningClassFilter.
	UPROPERTY(Config, EditAnywhere, Category = "Persisted Properties", DisplayName = "Properties")
	TArray<FLevelStreamingPersistentProperty> Properties;

	friend class ULevelStreamingPersistentPropertiesInfo;
};