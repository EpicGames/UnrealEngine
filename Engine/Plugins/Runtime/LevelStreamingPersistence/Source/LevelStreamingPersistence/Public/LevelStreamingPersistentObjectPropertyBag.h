// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/UnrealType.h"
#include "StructUtils/PropertyBag.h"

struct FLevelStreamingPersistentObjectPropertyBag
{
	FLevelStreamingPersistentObjectPropertyBag() = default;
	FLevelStreamingPersistentObjectPropertyBag(FLevelStreamingPersistentObjectPropertyBag&& InOther);
	bool Initialize(TFunctionRef<const UPropertyBag* ()> InFunc);
	bool IsValid() const { return PropertyBag.IsValid(); }
	const UPropertyBag* GetPropertyBagStruct() const { return PropertyBag.GetPropertyBagStruct(); }
	const FProperty* FindPropertyByName(const FName InPropertyName) const;
	const FProperty* CopyPropertyValueFromObject(const UObject* InObject, const FProperty* InObjectProperty);
	const FProperty* CopyPropertyValueFromPropertyBag(const FLevelStreamingPersistentObjectPropertyBag& InPropertyBag, const FProperty* InProperty);
	const FProperty* CopyPropertyValueToObject(UObject* InObject, const FProperty* InObjectProperty) const;
	const FProperty* GetCompatibleProperty(const FProperty* InObjectProperty) const;
	bool ComparePropertyValueWithObject(const UObject* InObject, const FProperty* InObjectProperty, bool& bOutIsIdentical) const;
	void ForEachProperty(TFunctionRef<void(const FProperty*)> Func) const;
	void DumpContent(TFunctionRef<void(const FProperty*, const FString&)> Func, const TArray<const FProperty*>* InDumpProperties = nullptr);
	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FLevelStreamingPersistentObjectPropertyBag& PropertyBag);

	template<typename PropertyType>
	bool SetPropertyValue(const FName InPropertyName, const PropertyType& InPropertyValue);

	template<typename PropertyType>
	bool GetPropertyValue(const FName InPropertyName, PropertyType& OutPropertyValue) const;

	bool SetPropertyValueFromString(const FName InPropertyName, const FString& InPropertyValue);
	bool GetPropertyValueAsString(const FName InPropertyName, FString& OutPropertyValueAsString) const;

private:
	template<typename PropertyType>
	EPropertyBagResult SetValue(const FName InPropertyName, const PropertyType& InPropertyValue);
	
	template<typename PropertyType>
	TValueOrError<PropertyType, EPropertyBagResult> GetValue(const FName InPropertyName) const;

	template<typename PropertyType>
	TValueOrError<PropertyType, EPropertyBagResult> GetValueStruct(const FName InPropertyName) const;

	template<typename PropertyType>
	struct TPropertyBagPropertyType { static const EPropertyBagPropertyType Value = EPropertyBagPropertyType::None; };

	FInstancedPropertyBag PropertyBag;
};

template<typename PropertyType>
TValueOrError<PropertyType, EPropertyBagResult> FLevelStreamingPersistentObjectPropertyBag::GetValueStruct(const FName InPropertyName) const
{
	TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(InPropertyName);
	if (Result.HasValue() && Result.GetValue().GetPtr<PropertyType>())
	{
		return MakeValue(Result.GetValue().Get<PropertyType>());
	}
	return MakeError(Result.HasError() ? Result.GetError() : EPropertyBagResult::PropertyNotFound);
}

template<typename PropertyType>
bool FLevelStreamingPersistentObjectPropertyBag::SetPropertyValue(const FName InPropertyName, const PropertyType& InPropertyValue)
{
	if (IsValid())
	{
		const FPropertyBagPropertyDesc* Desc = PropertyBag.FindPropertyDescByName(InPropertyName);
		if (Desc && (Desc->ValueType == TPropertyBagPropertyType<PropertyType>::Value))
		{
			EPropertyBagResult Result = SetValue(InPropertyName, InPropertyValue);
			return Result == EPropertyBagResult::Success;
		}
	}
	return false;
}

template<typename PropertyType>
bool FLevelStreamingPersistentObjectPropertyBag::GetPropertyValue(const FName InPropertyName, PropertyType& OutPropertyValue) const
{
	if (IsValid())
	{
		const FPropertyBagPropertyDesc* Desc = PropertyBag.FindPropertyDescByName(InPropertyName);
		if (Desc && (Desc->ValueType == TPropertyBagPropertyType<PropertyType>::Value))
		{
			TValueOrError<PropertyType, EPropertyBagResult> Result = GetValue<PropertyType>(InPropertyName);
			if (Result.HasValue())
			{
				OutPropertyValue = Result.GetValue();
				return true;
			}
		}
	}
	return false;
}